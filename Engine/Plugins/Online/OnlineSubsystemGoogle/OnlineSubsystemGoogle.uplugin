{
	"FileVersion" : 3,
	
	"FriendlyName" : "Online Subsystem Google",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Access to Google platform",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,

	"Modules": [
		{
			"Name": "OnlineSubsystemGoogle",
			"Type": "Runtime",
			"LoadingPhase" : "Default",
			"PlatformAllowList" : [
				"Android",
				"IOS",
				"Linux",
				"LinuxArm64",
				"Mac",
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		}
	]

}