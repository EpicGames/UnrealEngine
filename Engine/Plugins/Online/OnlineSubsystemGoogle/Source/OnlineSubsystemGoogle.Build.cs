// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class OnlineSubsystemGoogle : ModuleRules
{
	protected virtual bool bUsesRestfulImpl
	{
		get =>
			Target.Platform == UnrealTargetPlatform.Win64 ||
			Target.Platform == UnrealTargetPlatform.Mac ||
			Target.IsInPlatformGroup(UnrealPlatformGroup.Unix);
	}

	public OnlineSubsystemGoogle(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateDefinitions.Add("ONLINESUBSYSTEMGOOGLE_PACKAGE=1");
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreOnline",
				"ApplicationCore",
				"HTTP",
				"Json",
				"OnlineSubsystem", 
			}
			);

		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
			PublicDefinitions.Add("WITH_GOOGLE=1");
			PrivateIncludePaths.Add("Private/IOS");

			// These are iOS system libraries that Google depends on
			PublicFrameworks.AddRange(
			new string[] {
				"CoreGraphics",
    			"CoreText",
    			"Foundation",
    			"LocalAuthentication",
				"SafariServices",
				"Security",
			});

			PublicWeakFrameworks.Add("AuthenticationServices");

			PCHUsage = ModuleRules.PCHUsageMode.NoPCHs;
			bEnableObjCAutomaticReferenceCounting = true;

			PublicAdditionalFrameworks.Add(
			new Framework(
				"GoogleSignIn",
				"ThirdParty/IOS/GoogleSignInSDK/GoogleSignIn.embeddedframework.zip",
				Framework.FrameworkMode.LinkAndCopy)
			);

			PublicAdditionalFrameworks.Add(
			new Framework(
				"AppAuth",
				"ThirdParty/IOS/GoogleSignInSDK/AppAuth.embeddedframework.zip",
				Framework.FrameworkMode.LinkAndCopy)
			);

			PublicAdditionalFrameworks.Add(
			new Framework(
				"GTMAppAuth",
				"ThirdParty/IOS/GoogleSignInSDK/GTMAppAuth.embeddedframework.zip",
				Framework.FrameworkMode.LinkAndCopy)
			);

			PublicAdditionalFrameworks.Add(
			new Framework(
				"GTMSessionFetcher",
				"ThirdParty/IOS/GoogleSignInSDK/GTMSessionFetcher.embeddedframework.zip",
				Framework.FrameworkMode.LinkAndCopy)
			);

			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("IOSPlugin", Path.Combine(PluginPath, "OnlineSubsystemGoogle_IOS_UPL.xml"));
		}
		else if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Launch",
			}
			);

			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "OnlineSubsystemGoogle_Android_UPL.xml"));

			PrivateIncludePaths.Add("Private/Android");
		}
		else if (bUsesRestfulImpl)
		{
			PrivateIncludePaths.Add("Private/Rest");
		}
		else
		{
			PrecompileForTargets = PrecompileTargetsType.None;
		}

		PublicDefinitions.Add("USES_RESTFUL_GOOGLE=" + (bUsesRestfulImpl ? "1" : "0"));
	}
}
