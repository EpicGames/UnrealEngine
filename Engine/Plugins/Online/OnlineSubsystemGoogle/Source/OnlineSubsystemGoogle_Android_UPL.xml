<?xml version="1.0" encoding="utf-8"?>
<!--OnlineSubsystemGoogle plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="Google SDK Android init"/>
        
		<setBoolFromProperty result="bEnabled" ini="Engine" section="OnlineSubsystemGoogle" property="bEnabled" default="false"/>
		<setIntFromProperty result="RequestCodeForGoogleLogin" ini="Engine" section="OnlineSubsystemGoogle.OnlineIdentityGoogle" property="RequestCodeForGoogleLogin" default="9001"/>
		<dumpvars/>
	</init>

	<registerBuildSettings>
	<insertValue value="OnlineSubsystemGoogle[bEnabled]"/>
	<insertValue value="OnlineSubsystemGoogle.OnlineIdentityGoogle[RequestCodeForGoogleLogin]"/>
	<insertNewline/>
	</registerBuildSettings>

	<!-- optional files or directories to copy to Intermediate/Android/APK -->
	<resourceCopies>
	</resourceCopies>
	
	<!-- Files to copy before the build so they can be used in the build -->
	<prebuildCopies>
		<if condition="bEnabled">
			<true>
				<log text="Copying Google buildtime files to staging"/>
				<copyDir src="$S(PluginDir)/ThirdParty/Android/Java"
					dst = "$S(BuildDir)/src/com/epicgames/unreal" /> 	
			</true>
		</if>
	</prebuildCopies>
	
    <buildGradleAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
	dependencies {
		implementation('androidx.credentials:credentials:1.2.2')
		implementation('androidx.credentials:credentials-play-services-auth:1.2.2')
		implementation('com.google.android.libraries.identity.googleid:googleid:1.1.0')
	}
				</insert>
			</true>
		</if>
    </buildGradleAdditions>

	<!-- optional updates applied to AndroidManifest.xml -->
	<androidManifestUpdates>
	</androidManifestUpdates>

	<!-- optional additions to proguard -->
	<proguardAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
 -keep class com.epicgames.unreal.GoogleLogin {
	public *;
}                   
-if class androidx.credentials.CredentialManager
-keep class androidx.credentials.playservices.** {
  *;
}
				</insert>
			</true>
		</if>
	</proguardAdditions>

	<!-- optional additions to the GameActivity class in GameActivity.java -->
	<gameActivityClassAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
	// Start Google gameActivityClassAdditions
				</insert>
				<insertValue value="public static int REQUEST_CODE_OSSGOOGLE_LOGIN = $I(RequestCodeForGoogleLogin);"/>
				<insertNewline/>
				<insert>
	private GoogleLogin googleLogin;
	public GoogleLogin getGoogleLogin() 
	{ 
		return googleLogin; 
	}
	// End Google gameActivityClassAdditions
				</insert>
			</true>
		</if>
	</gameActivityClassAdditions>
	
	<!-- optional additions to GameActivity onCreate in GameActivity.java -->
	<gameActivityOnCreateAdditions>
		<if condition="bEnabled">
			<true>
				<insert>	
	// Begin Google onCreate
    Log.debug("[JAVA] - Google onCreate");
    googleLogin = new GoogleLogin(this);
	// End Google onCreate
				</insert>
			</true>
		</if>
	</gameActivityOnCreateAdditions>

	<!-- optional additions to GameActivity onActivityResult in GameActivity.java -->
	<gameActivityOnActivityResultAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
		// Begin Google onActivityResult
		Log.debug("[JAVA] - Google onActivityResult");
		if (requestCode == REQUEST_CODE_OSSGOOGLE_LOGIN)
		{
			googleLogin.onActivityResult(resultCode, data);
		}
		// End Google onActivityResult
				</insert>
			</true>
		</if>
	</gameActivityOnActivityResultAdditions>

	<!-- optional libraries to load in GameActivity.java before libUnreal.so -->
	<soLoadLibrary>
	</soLoadLibrary>
</root>
