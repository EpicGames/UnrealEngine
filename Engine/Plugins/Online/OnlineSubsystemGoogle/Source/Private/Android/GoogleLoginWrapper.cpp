// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleLoginWrapper.h"

#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#include "OnlineSubsystemGoogle.h"
#include "OnlineIdentityGoogle.h"
#include "OnlineJniGoogleLogin.h"

bool FGoogleLoginWrapper::Init(const FString& ServerClientId, bool bRequestServerAuthCode)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		if (!GoogleLoginInstance)
		{
			static jmethodID GetGoogleLoginMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "getGoogleLogin", "()Lcom/epicgames/unreal/GoogleLogin;", false);
			if(!GetGoogleLoginMethod)
			{
				UE_LOG_ONLINE_IDENTITY(Error, TEXT("Failed to find getGoogleLogin in GameActivity class"));
				return false;
			}

			auto Instance = NewScopedJavaObject(Env, FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, GetGoogleLoginMethod));
			if (Env->ExceptionCheck())
			{
				UE_LOG_ONLINE_IDENTITY(Error, TEXT("Failed to get GoogleLogin instance"));
				Env->ExceptionDescribe();
				Env->ExceptionClear();
				return false;
			}

			GoogleLoginInstance = Env->NewGlobalRef(*Instance);

			InitMethod = FJavaWrapper::FindMethod(Env, UE::Jni::Class<UE::Jni::FGoogleLogin>, "Init", "(Ljava/lang/String;Z)Z", false);
			UE_CLOG_ONLINE_IDENTITY(InitMethod == nullptr, Error, TEXT("Failed to find Init in GoogleLogin class"));
			LoginMethod = FJavaWrapper::FindMethod(Env, UE::Jni::Class<UE::Jni::FGoogleLogin>, "Login", "([Ljava/lang/String;)V", false);
			UE_CLOG_ONLINE_IDENTITY(LoginMethod == nullptr, Error, TEXT("Failed to find Login in GoogleLogin class"));
			LogoutMethod = FJavaWrapper::FindMethod(Env, UE::Jni::Class<UE::Jni::FGoogleLogin>, "Logout", "()V", false);
			UE_CLOG_ONLINE_IDENTITY(LogoutMethod == nullptr, Error, TEXT("Failed to find Logout in GoogleLogin class"));
		}

		auto jServerClientId = FJavaHelper::ToJavaString(Env, ServerClientId);
		jboolean jbRequestServerAuthCode = bRequestServerAuthCode ? JNI_TRUE : JNI_FALSE;

		bool Ret = FJavaWrapper::CallBooleanMethod(Env, GoogleLoginInstance, InitMethod, *jServerClientId, jbRequestServerAuthCode);
		if (Env->ExceptionCheck())
		{
			UE_LOG_ONLINE_IDENTITY(Error, TEXT("Failed to call Init method"));
			Env->ExceptionDescribe();
			Env->ExceptionClear();
			return false;
		}
		return Ret;
	}
	return false;
}

void FGoogleLoginWrapper::Shutdown()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		if (GoogleLoginInstance)
		{
			Env->DeleteGlobalRef(GoogleLoginInstance);
		}
	}
	GoogleLoginInstance = nullptr;
}


bool FGoogleLoginWrapper::Login(const TArray<FString>& InScopeFields)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		if (GoogleLoginInstance && LoginMethod)
		{
			// Convert scope array into java fields
			auto ScopeIDArray = NewScopedJavaObject(Env, (jobjectArray)Env->NewObjectArray(InScopeFields.Num(), FJavaWrapper::JavaStringClass, nullptr));
			for (uint32 Param = 0; Param < InScopeFields.Num(); Param++)
			{
				auto StringValue = FJavaHelper::ToJavaString(Env, InScopeFields[Param]);
				Env->SetObjectArrayElement(*ScopeIDArray, Param, *StringValue);
			}

			FJavaWrapper::CallVoidMethod(Env, GoogleLoginInstance, LoginMethod, *ScopeIDArray);
			if (Env->ExceptionCheck())
			{
				Env->ExceptionDescribe();
				Env->ExceptionClear();
				return false;
			}
			return true;
		}		
	}
	UE_LOG_ONLINE_IDENTITY(Error, TEXT("Failed to call Login method in Java GoogleLogin class"));
	return false;
}

bool FGoogleLoginWrapper::Logout()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		if (GoogleLoginInstance && LogoutMethod)
		{
			FJavaWrapper::CallVoidMethod(Env, GoogleLoginInstance, LogoutMethod);
			if (Env->ExceptionCheck())
			{
				Env->ExceptionDescribe();
				Env->ExceptionClear();
				return false;
			}
			return true;
		}		
	}
	UE_LOG_ONLINE_IDENTITY(Error, TEXT("Failed to call Logout method in Java GoogleLogin class"));
	return false;
}

void JNICALL UE::Jni::FGoogleLogin::nativeLoginSuccess(JNIEnv* jenv, jobject thiz, jstring InUserId, jstring InGivenName, jstring InFamilyName, jstring InDisplayName, jstring InPhotoUrl, jstring InIdToken, jstring InServerAuthCode)
{
	auto UserId = FJavaHelper::FStringFromParam(jenv, InUserId);
	auto GivenName = FJavaHelper::FStringFromParam(jenv, InGivenName);
	auto FamilyName = FJavaHelper::FStringFromParam(jenv, InFamilyName);
	auto DisplayName = FJavaHelper::FStringFromParam(jenv, InDisplayName);
	auto PhotoUrl = FJavaHelper::FStringFromParam(jenv, InPhotoUrl);
	auto ServerAuthCode = FJavaHelper::FStringFromParam(jenv, InServerAuthCode);

	FAuthTokenGoogle AuthToken;
	AuthToken.IdToken = FJavaHelper::FStringFromParam(jenv, InIdToken);

	if (!AuthToken.IdToken.IsEmpty() && AuthToken.IdTokenJWT.Parse(AuthToken.IdToken))
	{
		AuthToken.AddAuthData(AUTH_ATTR_ID_TOKEN, AuthToken.IdToken);
	}

	if (!ServerAuthCode.IsEmpty())
	{
		AuthToken.AddAuthData(AUTH_ATTR_AUTHORIZATION_CODE, FString(ServerAuthCode));
	}

	TSharedRef<FUserOnlineAccountGoogle> User = MakeShared<FUserOnlineAccountGoogle>(MoveTemp(UserId), MoveTemp(GivenName), MoveTemp(FamilyName), MoveTemp(DisplayName), MoveTemp(PhotoUrl), AuthToken);

	if (auto SubsystemGoogle = static_cast<FOnlineSubsystemGoogle* const>(IOnlineSubsystem::Get(GOOGLE_SUBSYSTEM)))
	{
		SubsystemGoogle->ExecuteNextTick([SubsystemGoogle, User]()
		{
			if (FOnlineIdentityGooglePtr IdentityGoogleInt = StaticCastSharedPtr<FOnlineIdentityGoogle>(SubsystemGoogle->GetIdentityInterface()))
			{
				IdentityGoogleInt->OnLoginComplete(EGoogleLoginResponse::RESPONSE_OK, User);
			}
		});
	}
}

void JNICALL UE::Jni::FGoogleLogin::nativeLoginFailed(JNIEnv* jenv, jobject thiz, jint ResponseCode)
{
	EGoogleLoginResponse LoginResponse = (EGoogleLoginResponse)ResponseCode;
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("nativeLoginSuccess Response: %s"), ToString(LoginResponse));

	if (auto SubsystemGoogle = static_cast<FOnlineSubsystemGoogle* const>(IOnlineSubsystem::Get(GOOGLE_SUBSYSTEM)))
	{
		SubsystemGoogle->ExecuteNextTick([SubsystemGoogle, LoginResponse]()
		{
			if (FOnlineIdentityGooglePtr IdentityGoogleInt = StaticCastSharedPtr<FOnlineIdentityGoogle>(SubsystemGoogle->GetIdentityInterface()))
			{
				IdentityGoogleInt->OnLoginComplete(LoginResponse, nullptr);
			}
		});
	}
}

void JNICALL UE::Jni::FGoogleLogin::nativeLogoutComplete(JNIEnv* jenv, jobject thiz, jsize responseCode)
{
	EGoogleLoginResponse LogoutResponse = (EGoogleLoginResponse)responseCode;
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("nativeLogoutComplete Response: %s"), ToString(LogoutResponse));

	if (auto SubsystemGoogle = static_cast<FOnlineSubsystemGoogle* const>(IOnlineSubsystem::Get(GOOGLE_SUBSYSTEM)))
	{
		SubsystemGoogle->ExecuteNextTick([SubsystemGoogle, LogoutResponse]()
		{
			if (FOnlineIdentityGooglePtr IdentityGoogleInt = StaticCastSharedPtr<FOnlineIdentityGoogle>(SubsystemGoogle->GetIdentityInterface()))
			{
				IdentityGoogleInt->OnLogoutComplete(LogoutResponse);
			}
		});
	}
}
