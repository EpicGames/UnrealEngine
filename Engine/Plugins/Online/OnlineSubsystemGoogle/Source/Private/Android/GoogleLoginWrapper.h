// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
 #include "CoreTypes.h"
 #include "Containers/UnrealString.h"

 THIRD_PARTY_INCLUDES_START
 #include <jni.h>
 THIRD_PARTY_INCLUDES_END

/**
 * Google interface class with com.epicgames.unreal.GoogleLogin Java instance
 */
class FGoogleLoginWrapper
{
public:
	bool Init(const FString& ServerClientId, bool bRequestServerAuthCode);
	void Shutdown();

	bool Login(const TArray<FString>& ScopeFields);
	bool Logout();

private:
	jobject GoogleLoginInstance = nullptr;
	jmethodID InitMethod = nullptr;
	jmethodID LoginMethod = nullptr;
	jmethodID LogoutMethod = nullptr;
};
