// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineExternalUIInterfaceGoogle.h"
#include "OnlineSubsystemGoogle.h"
#include "OnlineIdentityGoogle.h"
#include "OnlineError.h"


bool FOnlineExternalUIGoogle::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	IOnlineIdentityPtr OnlineIdentity = GoogleSubsystem->GetIdentityInterface();
	if (FUniqueNetIdPtr UniqueNetId = OnlineIdentity->GetUniquePlayerId(ControllerIndex); UniqueNetId && UniqueNetId->IsValid())
	{
		Delegate.ExecuteIfBound(UniqueNetId, ControllerIndex, FOnlineError::Success());
		return true;
	}
	TSharedPtr<FDelegateHandle> DelegateHandle = MakeShared<FDelegateHandle>();
	*DelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(ControllerIndex, FOnLoginCompleteDelegate::CreateLambda([this, DelegateHandle, Delegate](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& ErrorString)
		{
			FOnlineError Error(bWasSuccessful);
			Error.SetFromErrorCode(ErrorString);
			Delegate.ExecuteIfBound(UserId.IsValid()? UserId.AsShared() : FUniqueNetIdPtr(), LocalUserNum, Error);
			IOnlineIdentityPtr OnlineIdentity = GoogleSubsystem->GetIdentityInterface();
			OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(LocalUserNum, *DelegateHandle);
		}));
	OnlineIdentity->Login(ControllerIndex, FOnlineAccountCredentials());
	return true;
}
