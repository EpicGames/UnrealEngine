// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemGoogle.h"
#include "OnlineExternalUIGoogleCommon.h"
#include "OnlineSubsystemGooglePackage.h"

class FOnlineSubsystemGoogle;

/** 
 * Implementation for the Google external UIs
 */
class FOnlineExternalUIGoogle : public FOnlineExternalUIGoogleCommon
{

PACKAGE_SCOPE:

	/** 
	 * Constructor
	 * @param InSubsystem The owner of this external UI interface.
	 */
	FOnlineExternalUIGoogle(FOnlineSubsystemGoogle* InSubsystem) :
		FOnlineExternalUIGoogleCommon(InSubsystem)
	{
	}

public:

	/**
	 * Destructor.
	 */
	virtual ~FOnlineExternalUIGoogle()
	{
	}

	// IOnlineExternalUI
	virtual bool ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate = FOnLoginUIClosedDelegate()) override;


};

typedef TSharedPtr<FOnlineExternalUIGoogle, ESPMode::ThreadSafe> FOnlineExternalUIGooglePtr;

