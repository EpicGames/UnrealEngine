// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityGoogle.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "OnlineSubsystemGooglePrivate.h"

#include "Misc/ConfigCacheIni.h"

#define GOOGLE_JNI_CPP_ERROR -2
#define GOOGLE_JNI_JAVA_ERROR -1
#define GOOGLE_JNI_OK 0

FOnlineIdentityGoogle::FOnlineIdentityGoogle(FOnlineSubsystemGoogle* InSubsystem)
	: FOnlineIdentityGoogleCommon(InSubsystem)
{
	bAccessTokenAvailableToPlatform = false;
	
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityGoogle::FOnlineIdentityGoogle()"));

	// Setup permission scope fields
	GConfig->GetArray(TEXT("OnlineSubsystemGoogle.OnlineIdentityGoogle"), TEXT("ScopeFields"), ScopeFields, GEngineIni);
	// always required login access fields
	ScopeFields.AddUnique(TEXT(GOOGLE_PERM_PUBLIC_PROFILE));
}

bool FOnlineIdentityGoogle::Init()
{
	if (ensure(GoogleSubsystem))
	{
		bool bRequestServerAuthCode = ShouldRequestOfflineAccess();

		FString ServerClientId = GoogleSubsystem->GetServerClientId();

		if (ServerClientId.IsEmpty())
		{
			UE_LOG_ONLINE_IDENTITY(Error, TEXT("Init failed: ServerClientId not found in config"));
			return false;
		}

		return GoogleLoginWrapper.Init(ServerClientId, bRequestServerAuthCode);
	}

	return false;
}

void FOnlineIdentityGoogle::Shutdown()
{
	GoogleLoginWrapper.Shutdown();
}

FUniqueNetIdPtr FOnlineIdentityGoogle::RemoveUserId(int LocalUserNum) 
{
    FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
    if (UserId.IsValid())
    {
        // remove cached user account
        UserAccounts.Remove(UserId->ToString());
    }
    else
    {
        UserId = FUniqueNetIdGoogle::EmptyId();
    }
    // remove cached user id
    UserIds.Remove(LocalUserNum);
    return UserId;
}

bool FOnlineIdentityGoogle::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnlineIdentityGoogle::Login"));
	bool bTriggeredLogin = false;
	bool bPendingOp = LoginCompletionDelegate.IsBound() || LogoutCompletionDelegate.IsBound();
	if (!bPendingOp)
	{
		ELoginStatus::Type LoginStatus = GetLoginStatus(LocalUserNum);
		if (LoginStatus == ELoginStatus::NotLoggedIn)
		{
			LoginCompletionDelegate = FOnInternalLoginComplete::CreateLambda(
				[this, LocalUserNum](EGoogleLoginResponse InResponseCode, TSharedPtr<FUserOnlineAccountGoogle> User)
			{
				UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnInternalLoginComplete %s"), ToString(InResponseCode));
				if (InResponseCode == EGoogleLoginResponse::RESPONSE_OK)
				{
					// update/add cached entry for user
					UserAccounts.Add(User->GetUserId()->ToString(), User.ToSharedRef());
					// keep track of user ids for local users
					UserIds.Add(LocalUserNum, User->GetUserId());

					OnLoginAttemptComplete(LocalUserNum, "");
				}
				else
				{
					FString ErrorStr;
					if (InResponseCode == EGoogleLoginResponse::RESPONSE_CANCELED)
					{
						ErrorStr = LOGIN_CANCELLED;
					}
					else
					{
						ErrorStr = FString::Printf(TEXT("Login failure %s"), ToString(InResponseCode));
					}
					OnLoginAttemptComplete(LocalUserNum, ErrorStr);
				}
			});

			bTriggeredLogin = GoogleLoginWrapper.Login(ScopeFields);
			if (!bTriggeredLogin)
			{
				OnLoginComplete(EGoogleLoginResponse::RESPONSE_ERROR, nullptr);
				return false;
			}
		}
		else
		{
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, *GetUniquePlayerId(LocalUserNum), TEXT("Already logged in"));
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnlineIdentityGoogle::Login Operation already in progress!"));
		FString ErrorStr = FString::Printf(TEXT("Operation already in progress"));
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdGoogle::EmptyId(), ErrorStr);
	}

	return bTriggeredLogin;
}

void FOnlineIdentityGoogle::OnLoginAttemptComplete(int32 LocalUserNum, const FString& ErrorStr)
{
	if (GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Google login was successful."));
		FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
		check(UserId.IsValid());

		GoogleSubsystem->ExecuteNextTick([this, UserId, LocalUserNum]()
		{
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, *UserId, FString());
			TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, *UserId);
		});
	}
	else
	{
		LogoutCompletionDelegate = FOnInternalLogoutComplete::CreateLambda(
			[this, LocalUserNum, ErrorStr](EGoogleLoginResponse InResponseCode)
		{
			UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google login failed: %s"), *ErrorStr);

            FUniqueNetIdPtr UserId = RemoveUserId(LocalUserNum);

			TriggerOnLoginCompleteDelegates(LocalUserNum, false, *UserId, ErrorStr);
		});

		if (!GoogleLoginWrapper.Logout())
		{
			OnLogoutComplete(EGoogleLoginResponse::RESPONSE_ERROR);
		}
	}
}

bool FOnlineIdentityGoogle::Logout(int32 LocalUserNum)
{
	bool bTriggeredLogout = false;
	bool bPendingOp = LoginCompletionDelegate.IsBound() || LogoutCompletionDelegate.IsBound();
	if (!bPendingOp)
	{
		ELoginStatus::Type LoginStatus = GetLoginStatus(LocalUserNum);
		if (LoginStatus == ELoginStatus::LoggedIn)
		{
			LogoutCompletionDelegate = FOnInternalLogoutComplete::CreateLambda(
				[this, LocalUserNum](EGoogleLoginResponse InResponseCode)
			{
				UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnInternalLogoutComplete %s"), ToString(InResponseCode));
                FUniqueNetIdPtr UserId = RemoveUserId(LocalUserNum);

				GoogleSubsystem->ExecuteNextTick([this, UserId, LocalUserNum]()
				{
					TriggerOnLogoutCompleteDelegates(LocalUserNum, true);
					TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::LoggedIn, ELoginStatus::NotLoggedIn, *UserId);
				});
			});

			bTriggeredLogout = GoogleLoginWrapper.Logout();
			if (!bTriggeredLogout)
			{
				OnLogoutComplete(EGoogleLoginResponse::RESPONSE_ERROR);
				return false;
			}
		}
		else
		{
			UE_LOG_ONLINE_IDENTITY(Warning, TEXT("No logged in user found for LocalUserNum=%d."), LocalUserNum);
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FOnlineIdentityGoogle::Logout - Operation already in progress"));
	}

	if (!bTriggeredLogout)
	{
		UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnlineIdentityGoogle::Logout didn't trigger logout"));
		GoogleSubsystem->ExecuteNextTick([this, LocalUserNum]()
		{
			TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
		});
	}

	return bTriggeredLogout;
}

void FOnlineIdentityGoogle::OnLoginComplete(EGoogleLoginResponse InResponseCode, const TSharedPtr<FUserOnlineAccountGoogle>& User)
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("OnLoginComplete %s"), ToString(InResponseCode));
	ensure(LoginCompletionDelegate.IsBound());
	LoginCompletionDelegate.ExecuteIfBound(InResponseCode, User);
	LoginCompletionDelegate.Unbind();
}

void FOnlineIdentityGoogle::OnLogoutComplete(EGoogleLoginResponse InResponseCode)
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("OnLogoutComplete %s"), ToString(InResponseCode));
	ensure(LogoutCompletionDelegate.IsBound());
	LogoutCompletionDelegate.ExecuteIfBound(InResponseCode);
	LogoutCompletionDelegate.Unbind();
}
