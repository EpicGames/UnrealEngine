// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "OnlineIdentityGoogleCommon.h"
#include "OnlineSubsystemGoogleTypes.h"
#include "OnlineAccountGoogleCommon.h"
#include "GoogleLoginWrapper.h"
#include "OnlineSubsystemGooglePackage.h"


class FOnlineSubsystemGoogle;

/** Tied to GoogleLogin.java */
enum class EGoogleLoginResponse : uint8
{
	/** Google Sign In SDK ok response */
	RESPONSE_OK = 0,
	/** Google Sign In  SDK user cancellation */
	RESPONSE_CANCELED = 1,
	/** Google Sign In  SDK error */
	RESPONSE_ERROR = 2,
	/** Developer setup error */
	RESPONSE_DEVELOPER_ERROR
};

inline const TCHAR* ToString(EGoogleLoginResponse Response)
{
	switch (Response)
	{
		case EGoogleLoginResponse::RESPONSE_OK:
			return TEXT("RESPONSE_OK");
		case EGoogleLoginResponse::RESPONSE_CANCELED:
			return TEXT("RESPONSE_CANCELED");
		case EGoogleLoginResponse::RESPONSE_ERROR:
			return TEXT("RESPONSE_ERROR");
		case EGoogleLoginResponse::RESPONSE_DEVELOPER_ERROR:
			return TEXT("RESPONSE_DEVELOPER_ERROR");
	}

	return TEXT("");
}

/** Android implementation of a Google user account */
class FUserOnlineAccountGoogle : public FUserOnlineAccountGoogleCommon
{
public:

	explicit FUserOnlineAccountGoogle()
	{
	}

	explicit FUserOnlineAccountGoogle(FString&& InUserId, 
									  FString&& InGivenName,
									  FString&& InFamilyName,
									  FString&& InDisplayName,
									  FString&& InPicture,
									  const FAuthTokenGoogle& InAuthToken)
		: FUserOnlineAccountGoogleCommon(InUserId, InAuthToken)
	{
		RealName = InDisplayName;
		FirstName = InGivenName;
		LastName = InFamilyName;
		SetAccountData(TEXT("sub"), MoveTemp(InUserId));
		SetAccountData(TEXT("given_name"), MoveTemp(InGivenName));
		SetAccountData(TEXT("family_name"), MoveTemp(InFamilyName));
		SetAccountData(TEXT("name"), MoveTemp(InDisplayName));
		if (!InPicture.IsEmpty())
		{
			SetAccountData(TEXT("picture"), MoveTemp(InPicture));
		}
	}

	virtual ~FUserOnlineAccountGoogle()
	{
	}
};

/**
 * Google service implementation of the online identity interface
 */
class FOnlineIdentityGoogle :
	public FOnlineIdentityGoogleCommon
{

public:
	// IOnlineIdentity
	
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;

	// FOnlineIdentityGoogle

	FOnlineIdentityGoogle(FOnlineSubsystemGoogle* InSubsystem);

	/**	
	 * Initialize the interface
	 * 
	 * @return true if successful, false if there was an issue
	 */
	bool Init();

	/**	
	 * Shuts down the interface
	 * 
	 */
	void Shutdown();

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityGoogle()
	{
	}

PACKAGE_SCOPE:

	/** Last function called for any single login attempt */
	void OnLoginAttemptComplete(int32 LocalUserNum, const FString& ErrorStr);

	/** Generic handler for the Java SDK login callback */
	void OnLoginComplete(EGoogleLoginResponse InResponseCode, const TSharedPtr<FUserOnlineAccountGoogle>& User);
	/** Generic handler for the Java SDK logout callback */
	void OnLogoutComplete(EGoogleLoginResponse InResponseCode);
	
private:

	FGoogleLoginWrapper GoogleLoginWrapper;

	/** Config based list of permission scopes to use when logging in */
	TArray<FString> ScopeFields;

	/** Delegate holder for all internal related login callbacks */
	DECLARE_DELEGATE_TwoParams(FOnInternalLoginComplete, EGoogleLoginResponse /*InLoginResponse*/, TSharedPtr<FUserOnlineAccountGoogle> /*User*/);
	FOnInternalLoginComplete LoginCompletionDelegate;
	/** Delegate holder for all internal related logout callbacks */
	DECLARE_DELEGATE_OneParam(FOnInternalLogoutComplete, EGoogleLoginResponse /*InLoginResponse*/);
	FOnInternalLogoutComplete LogoutCompletionDelegate;

	FUniqueNetIdPtr RemoveUserId(int LocalUserNum);
};

typedef TSharedPtr<FOnlineIdentityGoogle, ESPMode::ThreadSafe> FOnlineIdentityGooglePtr;