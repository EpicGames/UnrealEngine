// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"

namespace UE::Jni
{
	struct FGoogleLogin: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/GoogleLogin";

		static void JNICALL nativeLoginSuccess(JNIEnv* env, jobject thiz, jstring InUserId, jstring InGivenName, jstring InFamilyName, jstring InDisplayName, jstring InPhotoUrl, jstring InIdToken, jstring InServerAuthCode);
		static void JNICALL nativeLoginFailed(JNIEnv* env, jobject thiz, jint ResponseCode);
		static void JNICALL nativeLogoutComplete(JNIEnv* env, jobject thiz, jsize responseCode);

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeLoginSuccess),
			UE_JNI_NATIVE_METHOD(nativeLoginFailed),
			UE_JNI_NATIVE_METHOD(nativeLogoutComplete)
		};
	};
	
	template struct TInitialize<FGoogleLogin>;
}
#endif
