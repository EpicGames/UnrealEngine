// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemGoogle.h"
#include "OnlineSubsystemGooglePrivate.h"
#include "OnlineIdentityGoogle.h"
#include "OnlineExternalUIInterfaceGoogle.h"

#include "Misc/ConfigCacheIni.h"

FOnlineSubsystemGoogle::FOnlineSubsystemGoogle(FName InInstanceName)
	: FOnlineSubsystemGoogleCommon(InInstanceName)
{
	bPlatformRequiresClientId = false;
	bPlatformAllowsClientIdOverride = false;	
	bPlatformRequiresServerClientId = true;
}

FOnlineSubsystemGoogle::~FOnlineSubsystemGoogle()
{
}

bool FOnlineSubsystemGoogle::Init()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemGoogle::Init()"));
	if (FOnlineSubsystemGoogleCommon::Init())
	{
		FOnlineIdentityGooglePtr TempPtr = MakeShareable(new FOnlineIdentityGoogle(this));
		if (TempPtr->Init())
		{
			GoogleIdentity = TempPtr;
		}

		GoogleExternalUI = MakeShareable(new FOnlineExternalUIGoogle(this));
	}

	return GoogleIdentity.IsValid() && GoogleExternalUI.IsValid();
}

bool FOnlineSubsystemGoogle::Shutdown()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemGoogle::Shutdown()"));

	if (auto Identity = StaticCastSharedPtr<FOnlineIdentityGoogle>(GoogleIdentity))
	{
		Identity->Shutdown();
	}

	return FOnlineSubsystemGoogleCommon::Shutdown();
}
