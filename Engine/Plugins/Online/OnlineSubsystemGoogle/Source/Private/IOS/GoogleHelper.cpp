// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleHelper.h"
#include "OnlineSubsystemGooglePrivate.h"
#include "Interfaces/OnlineIdentityInterface.h"

#include "Misc/CoreDelegates.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSAsyncTask.h"


bool GetAuthTokenFromSignInResult(GIDGoogleUser* User, NSString* ServerAuthCode, FAuthTokenGoogle& OutAuthToken)
{
	bool bSuccess = false;

	OutAuthToken.AccessToken = User.accessToken.tokenString;
	if (!OutAuthToken.AccessToken.IsEmpty())
	{
		OutAuthToken.IdToken = User.idToken.tokenString;
		if (!OutAuthToken.IdToken.IsEmpty() && OutAuthToken.IdTokenJWT.Parse(OutAuthToken.IdToken))
		{
			OutAuthToken.TokenType = TEXT("Bearer");
			OutAuthToken.ExpiresIn = 3600.0;
			OutAuthToken.RefreshToken = User.refreshToken.tokenString;

			OutAuthToken.AddAuthData(AUTH_ATTR_REFRESH_TOKEN, OutAuthToken.RefreshToken);
			OutAuthToken.AddAuthData(TEXT("access_token"), OutAuthToken.AccessToken);
			OutAuthToken.AddAuthData(AUTH_ATTR_ID_TOKEN, OutAuthToken.IdToken);
			if (ServerAuthCode)
			{
				OutAuthToken.AddAuthData(AUTH_ATTR_AUTHORIZATION_CODE, FString(ServerAuthCode));
			}
			OutAuthToken.AuthType = EGoogleAuthTokenType::AccessToken;
			OutAuthToken.ExpiresInUTC = FDateTime::UtcNow() + FTimespan(OutAuthToken.ExpiresIn * ETimespan::TicksPerSecond);
			bSuccess = true;
		}
		else
		{
			UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("GetAuthTokenFromGoogleUser: Failed to parse id token"));
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("GetAuthTokenFromGoogleUser: Access token missing"));
	}		

	return bSuccess;
}

@implementation FGoogleHelper

- (id)init
{
	self = [super init];

	dispatch_async(dispatch_get_main_queue(), ^
	{
		[GIDSignIn.sharedInstance restorePreviousSignInWithCompletion:^(GIDGoogleUser* User, NSError* Error)
		 {
			UE_CLOG_ONLINE_IDENTITY(User != nil, Display, TEXT("Restored previous sign in"));
			[self PrintAuthStatus];
		 }];
	});

	return self;
}

-(FDelegateHandle)AddOnGoogleSignInComplete: (const FOnGoogleSignInCompleteDelegate&) Delegate
{
	_OnSignInComplete.Add(Delegate);
	return Delegate.GetHandle();
}

-(FDelegateHandle)AddOnGoogleSignOutComplete: (const FOnGoogleSignOutCompleteDelegate&) Delegate
{
	_OnSignOutComplete.Add(Delegate);
	return Delegate.GetHandle();
}

- (void)ShowLoginUI: (NSArray*) InScopes
{
	[GIDSignIn.sharedInstance signInWithPresentingViewController:(UIViewController*)[IOSAppDelegate GetDelegate].IOSController
							hint:nil
							additionalScopes:InScopes
							completion:^(GIDSignInResult* SignInResult, NSError* Error)
							{
								[self DidSignIn: SignInResult.user withServerAuthCode: SignInResult.serverAuthCode withError: Error];
							}];
}

- (void)Login: (NSArray*) InScopes attemptSilentSignIn:(bool) bAttemptSilentSignIn
{
	[self PrintAuthStatus];

	dispatch_async(dispatch_get_main_queue(), ^
	{
		GIDGoogleUser* User = [GIDSignIn.sharedInstance currentUser];
		if (User == nil || !bAttemptSilentSignIn)
		{
			[self ShowLoginUI: InScopes];
		}
		else
		{
			NSMutableArray *MissingScopes = [NSMutableArray arrayWithArray:InScopes];
			[MissingScopes removeObjectsInArray: User.grantedScopes];
			if (MissingScopes.count == 0)
			{
				[User refreshTokensIfNeededWithCompletion: ^(GIDGoogleUser* SignedInUser, NSError* Error)
				 {
					if (Error != nil)
					{
						[self ShowLoginUI: InScopes];
					}
					else
					{
						[self DidSignIn: SignedInUser withServerAuthCode: nil withError: nil];
					}
				}];
			}
			else
			{
				[User addScopes: MissingScopes 
					 presentingViewController: (UIViewController*)[IOSAppDelegate GetDelegate].IOSController
					 completion: ^(GIDSignInResult* Result, NSError* Error)
				{
					[self DidSignIn: Result.user withServerAuthCode: Result.serverAuthCode withError: Error];
				}];
			}
		}
	});

}

- (void)Logout
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("logout"));

	dispatch_async(dispatch_get_main_queue(), ^
	{
		[[GIDSignIn sharedInstance] disconnectWithCompletion: ^(NSError* Error)
		 {
			[self DidDisconnect: Error];
		 }];
	});
}

- (void)DidSignIn:(GIDGoogleUser*)User withServerAuthCode:(NSString*)ServerAuthCode withError:(NSError *)Error
{
	FGoogleSignInData SignInData;
	if (Error != nil)
	{
		SignInData.ErrorStr = FString([Error localizedDescription]);
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("didSignInWithError Error: %s"), *SignInData.ErrorStr);

		if (Error.code == kGIDSignInErrorCodeHasNoAuthInKeychain)
		{
			SignInData.Response = EGoogleLoginResponse::RESPONSE_NOAUTH;
		}
		else if (Error.code == kGIDSignInErrorCodeCanceled)
		{
			SignInData.Response = EGoogleLoginResponse::RESPONSE_CANCELED;
		}
		else
		{
			SignInData.Response = EGoogleLoginResponse::RESPONSE_ERROR;
		}
	}
	else
	{
		if (GetAuthTokenFromSignInResult(User, ServerAuthCode, SignInData.AuthToken))
		{
			SignInData.Response = EGoogleLoginResponse::RESPONSE_OK;
		}
		else
		{
			SignInData.Response = EGoogleLoginResponse::RESPONSE_ERROR;
		}
	}

	UE_LOG_ONLINE_IDENTITY(Display, TEXT("SignIn: %s"), *SignInData.ToDebugString());

	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
		// Notify on the game thread
		_OnSignInComplete.Broadcast(SignInData);
		return true;
	}];
}

- (void)DidDisconnect:(NSError *)Error
{
	FGoogleSignOutData SignOutData;
	if (Error != nil)
	{
		SignOutData.ErrorStr = FString([Error localizedDescription]);
		SignOutData.Response = EGoogleLoginResponse::RESPONSE_ERROR;
	}
	else
	{
		SignOutData.Response = EGoogleLoginResponse::RESPONSE_OK;
	}
	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("didDisconnectWithUser Complete: %s"), *SignOutData.ToDebugString());

		// Notify on the game thread
		_OnSignOutComplete.Broadcast(SignOutData);
		return true;
	}];
}

- (void)PrintAuthStatus
{
	bool bHasAuth = [[GIDSignIn sharedInstance] hasPreviousSignIn];
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("HasAuth: %d"), bHasAuth);

	GIDGoogleUser* GoogleUser = [[GIDSignIn sharedInstance] currentUser];
	if (GoogleUser)
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Authentication:"));
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("- Access: %s"), *FString(GoogleUser.accessToken.tokenString));
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("- Refresh: %s"), *FString(GoogleUser.refreshToken.tokenString));
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Scopes:"));
		for (NSString* scope in GoogleUser.grantedScopes)
		{
			UE_LOG_ONLINE_IDENTITY(Display, TEXT("- %s"), *FString(scope));
		}

		UE_LOG_ONLINE_IDENTITY(Display, TEXT("User:"));
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("- UserId: %s RealName: %s FirstName: %s LastName: %s Email: %s"),
			   *FString(GoogleUser.userID),
			   *FString(GoogleUser.profile.name),
			   *FString(GoogleUser.profile.givenName),
			   *FString(GoogleUser.profile.familyName),
			   *FString(GoogleUser.profile.email));
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Authentication:"));
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("- None"));
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("User:"));
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("- None"));
	}
}

@end
