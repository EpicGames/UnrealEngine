// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineSubsystemGoogleTypes.h"

THIRD_PARTY_INCLUDES_START
#import <GoogleSignIn/GoogleSignIn.h>
THIRD_PARTY_INCLUDES_END

enum class EGoogleLoginResponse : uint8
{
	/** Google Sign In SDK ok response */
	RESPONSE_OK = 0,
	/** Silent sign in failed */
	RESPONSE_NOAUTH = 1,
	/** Google Sign In  SDK user cancellation */
	RESPONSE_CANCELED = 2,
	/** Google Sign In  SDK error */
	RESPONSE_ERROR = 3
};

inline const TCHAR* ToString(EGoogleLoginResponse Response)
{
	switch (Response)
	{
		case EGoogleLoginResponse::RESPONSE_OK:
			return TEXT("RESPONSE_OK");
		case EGoogleLoginResponse::RESPONSE_NOAUTH:
			return TEXT("RESPONSE_NOAUTH");
		case EGoogleLoginResponse::RESPONSE_CANCELED:
			return TEXT("RESPONSE_CANCELED");
		case EGoogleLoginResponse::RESPONSE_ERROR:
			return TEXT("RESPONSE_ERROR");
		default:
			break;
	}

	return TEXT("");
}


struct FGoogleSignInData
{
	/** @return a string that prints useful debug information about this response */
	FString ToDebugString() const
	{
		return FString::Printf(TEXT("Response: %s Valid: %d Error: %s"),
							   ToString(Response),
							   AuthToken.IsValid(),
							   *ErrorStr);
	}

	/** Result of the sign in */
	EGoogleLoginResponse Response;
	/** Error response, if any */
	FString ErrorStr;
	/** Token data (access/refresh/id), only valid on RESPONSE_OK */
	FAuthTokenGoogle AuthToken;
};

struct FGoogleSignOutData
{
	/** @return a string that prints useful debug information about this response */
	FString ToDebugString() const
	{
		return FString::Printf(TEXT("Response: %s Error: %s"),
							   ToString(Response),
							   *ErrorStr);
	}

	/** Result of the sign in */
	EGoogleLoginResponse Response;
	/** Error response, if any */
	FString ErrorStr;
};

/**
 * Delegate fired when Google sign in has completed
 *
 * @param SignInData
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGoogleSignInComplete, const FGoogleSignInData& /* SignInData */);
typedef FOnGoogleSignInComplete::FDelegate FOnGoogleSignInCompleteDelegate;

/**
 * Delegate fired when Google sign out has completed
 *
 * @param SignOutData
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGoogleSignOutComplete, const FGoogleSignOutData& /* SignOutData */);
typedef FOnGoogleSignOutComplete::FDelegate FOnGoogleSignOutCompleteDelegate;

@interface FGoogleHelper : NSObject
{
	FOnGoogleSignInComplete _OnSignInComplete;
	FOnGoogleSignOutComplete _OnSignOutComplete;
};

- (id)init;
- (void) Login: (NSArray*) InScopes attemptSilentSignIn:(bool) bAttemptSilentSignIn;
- (void) Logout;

/** Add a listener to the sign in complete event */
-(FDelegateHandle)AddOnGoogleSignInComplete: (const FOnGoogleSignInCompleteDelegate&) Delegate;
/** Add a listener to the sign out complete event */
-(FDelegateHandle)AddOnGoogleSignOutComplete: (const FOnGoogleSignOutCompleteDelegate&) Delegate;

@end
