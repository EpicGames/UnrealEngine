// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityGoogle.h"

#if PLATFORM_APPLE
#include "Apple/AppleStringUtils.h"
#endif

#include "OnlineSubsystemGooglePrivate.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "GoogleHelper.h"

#include "Misc/ConfigCacheIni.h"

FOnlineIdentityGoogle::FOnlineIdentityGoogle(FOnlineSubsystemGoogle* InSubsystem)
	: FOnlineIdentityGoogleCommon(InSubsystem)
	, bAllowSilentSignIn(false)
{
	GConfig->GetBool(TEXT("OnlineSubsystemGoogle.OnlineIdentityGoogle"), TEXT("bAllowedSilentSignIn"), bAllowSilentSignIn, GEngineIni);

	// Setup permission scope fields
	GConfig->GetArray(TEXT("OnlineSubsystemGoogle.OnlineIdentityGoogle"), TEXT("ScopeFields"), ScopeFields, GEngineIni);
	// always required login access fields
	ScopeFields.AddUnique(TEXT(GOOGLE_PERM_PUBLIC_PROFILE));
}

FOnlineIdentityGoogle::~FOnlineIdentityGoogle()
{
}

bool FOnlineIdentityGoogle::Init()
{
	if (ShouldRequestOfflineAccess())
	{
		NSString *ServerClientId = [[NSBundle mainBundle] objectForInfoDictionaryKey:@"GIDServerClientID"];
		UE_CLOG_ONLINE_IDENTITY([ServerClientId length] == 0, Warning, TEXT("ServerClientId not found in config. Server Auth Code won't be requested"));
	}

	GoogleHelper = [[FGoogleHelper alloc] init];

	FOnGoogleSignInCompleteDelegate OnSignInDelegate;
	OnSignInDelegate.BindRaw(this, &FOnlineIdentityGoogle::OnSignInComplete);
	[GoogleHelper AddOnGoogleSignInComplete: OnSignInDelegate];

	FOnGoogleSignOutCompleteDelegate OnSignOutDelegate;
	OnSignOutDelegate.BindRaw(this, &FOnlineIdentityGoogle::OnSignOutComplete);
	[GoogleHelper AddOnGoogleSignOutComplete: OnSignOutDelegate];

	return true;
}

void FOnlineIdentityGoogle::OnSignInComplete(const FGoogleSignInData& InSignInData)
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("OnSignInComplete %s"), ToString(InSignInData.Response));

	if (LoginCompletionDelegate.IsBound())
	{
		LoginCompletionDelegate.ExecuteIfBound(InSignInData.Response, InSignInData.AuthToken);
		LoginCompletionDelegate.Unbind();
	}
}

void FOnlineIdentityGoogle::OnSignOutComplete(const FGoogleSignOutData& InSignOutData)
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("OnSignOutComplete %s"), ToString(InSignOutData.Response));
	ensure(LogoutCompletionDelegate.IsBound());
	LogoutCompletionDelegate.ExecuteIfBound(InSignOutData.Response);
	LogoutCompletionDelegate.Unbind();
}

bool FOnlineIdentityGoogle::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnlineIdentityGoogle::Login"));

	bool bTriggeredLogin = false;
	bool bPendingOp = LoginCompletionDelegate.IsBound() || LogoutCompletionDelegate.IsBound();
	if (!bPendingOp)
	{
		ELoginStatus::Type LoginStatus = GetLoginStatus(LocalUserNum);
		if (LoginStatus == ELoginStatus::NotLoggedIn)
		{
			PendingLoginRequestCb PendingLoginFn = [this, LocalUserNum](bool bWasSuccessful)
			{
				if (bWasSuccessful)
				{
					NSMutableArray* Permissions = [[NSMutableArray alloc] initWithCapacity:ScopeFields.Num()];
					for (int32 ScopeIdx = 0; ScopeIdx < ScopeFields.Num(); ScopeIdx++)
					{
						NSString* ScopeStr = [NSString stringWithFString:ScopeFields[ScopeIdx]];
						[Permissions addObject: ScopeStr];
					}

					LoginCompletionDelegate = FOnInternalLoginComplete::CreateLambda([this, LocalUserNum](EGoogleLoginResponse InResponseCode, const FAuthTokenGoogle& InAccessToken)
	                {
						if (InResponseCode == EGoogleLoginResponse::RESPONSE_OK)
						{
							Login(LocalUserNum, InAccessToken, FOnLoginCompleteDelegate::CreateRaw(this, &FOnlineIdentityGoogle::OnAccessTokenLoginComplete));
						}
						else
						{
							FString ErrorStr;
							if (InResponseCode == EGoogleLoginResponse::RESPONSE_CANCELED)
							{
								ErrorStr = LOGIN_CANCELLED;
							}
							else
							{
								ErrorStr = FString::Printf(TEXT("Login failure %s"), ToString(InResponseCode));
							}

							OnLoginAttemptComplete(LocalUserNum, ErrorStr);
						}
					});

					[GoogleHelper Login: Permissions attemptSilentSignIn: bAllowSilentSignIn];
				}
				else
				{
					const FString ErrorStr = TEXT("Error retrieving discovery service");
					OnLoginAttemptComplete(LocalUserNum, ErrorStr);
				}
			};

			bTriggeredLogin = true;
			RetrieveDiscoveryDocument(MoveTemp(PendingLoginFn));
		}
		else
		{
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, *GetUniquePlayerId(LocalUserNum), TEXT("Already logged in"));
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnlineIdentityGoogle::Login Operation already in progress!"));
		FString ErrorStr = FString::Printf(TEXT("Operation already in progress"));
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdGoogle::EmptyId(), ErrorStr);
	}

	return bTriggeredLogin;
}

void FOnlineIdentityGoogle::Login(int32 LocalUserNum, const FAuthTokenGoogle& InToken, const FOnLoginCompleteDelegate& InCompletionDelegate)
{
	FOnProfileRequestComplete ProfileCompletionDelegate = FOnProfileRequestComplete::CreateLambda([this, InCompletionDelegate](int32 ProfileLocalUserNum, bool bWasSuccessful, const FString& ErrorStr)
	{
		if (bWasSuccessful)
		{
			InCompletionDelegate.ExecuteIfBound(ProfileLocalUserNum, bWasSuccessful, *GetUniquePlayerId(ProfileLocalUserNum), ErrorStr);
		}
		else
		{
			InCompletionDelegate.ExecuteIfBound(ProfileLocalUserNum, bWasSuccessful, *FUniqueNetIdGoogle::EmptyId(), ErrorStr);
		}
	});

	ProfileRequest(LocalUserNum, InToken, ProfileCompletionDelegate);
}

void FOnlineIdentityGoogle::OnAccessTokenLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UniqueId, const FString& Error)
{
	OnLoginAttemptComplete(LocalUserNum, Error);
}

void FOnlineIdentityGoogle::OnLoginAttemptComplete(int32 LocalUserNum, const FString& ErrorStr)
{
	const FString ErrorStrCopy(ErrorStr);
	if (GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Google login was successful"));
		FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
		check(UserId.IsValid());

		GoogleSubsystem->ExecuteNextTick([this, UserId, LocalUserNum, ErrorStrCopy]()
		{
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, *UserId, ErrorStrCopy);
			TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, *UserId);
		});
	}
	else
	{
		GoogleSubsystem->ExecuteNextTick([this, LocalUserNum, ErrorStrCopy]()
	    {
			TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdGoogle::EmptyId(), ErrorStrCopy);
		});
	}
}

bool FOnlineIdentityGoogle::Logout(int32 LocalUserNum)
{
	bool bTriggeredLogout = false;

	ELoginStatus::Type LoginStatus = GetLoginStatus(LocalUserNum);
	bool bPendingOp = LoginCompletionDelegate.IsBound() || LogoutCompletionDelegate.IsBound();
	if (!bPendingOp)
	{
		if (LoginStatus == ELoginStatus::LoggedIn)
		{
			LogoutCompletionDelegate = FOnInternalLogoutComplete::CreateLambda([this, LocalUserNum](EGoogleLoginResponse InResponseCode)
	        {
				UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnInternalLogoutComplete %s"), ToString(InResponseCode));
				FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
				if (UserId.IsValid())
				{
					// remove cached user account
					UserAccounts.Remove(UserId->ToString());
				}
				else
				{
					UserId = FUniqueNetIdGoogle::EmptyId();
				}
				// remove cached user id
				UserIds.Remove(LocalUserNum);

				GoogleSubsystem->ExecuteNextTick([this, UserId, LocalUserNum]()
			    {
					TriggerOnLogoutCompleteDelegates(LocalUserNum, true);
					TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::LoggedIn, ELoginStatus::NotLoggedIn, *UserId);
				});
			});

			bTriggeredLogout = true;
			[GoogleHelper Logout];
		}
		else
		{
			UE_LOG_ONLINE_IDENTITY(Warning, TEXT("No logged in user found for LocalUserNum=%d."), LocalUserNum);
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FOnlineIdentityGoogle::Logout - Operation already in progress"));
	}

	if (!bTriggeredLogout)
	{
		UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnlineIdentityGoogle::Logout didn't trigger logout"));
		GoogleSubsystem->ExecuteNextTick([this, LocalUserNum]()
	    {
			TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
		});
	}

	return bTriggeredLogout;
}

