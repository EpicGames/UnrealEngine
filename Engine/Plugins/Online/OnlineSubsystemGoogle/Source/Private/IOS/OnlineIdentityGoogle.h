// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "OnlineIdentityGoogleCommon.h"
#include "OnlineSubsystemGoogleTypes.h"
#include "OnlineAccountGoogleCommon.h"
#include "OnlineSubsystemGooglePackage.h"

struct FGoogleSignInData;
struct FGoogleSignOutData;
class FOnlineSubsystemGoogle;
enum class EGoogleLoginResponse : uint8;

@class FGoogleHelper;

/** iOS implementation of a Google user account */
class FUserOnlineAccountGoogle : public FUserOnlineAccountGoogleCommon
{
public:

	explicit FUserOnlineAccountGoogle(const FString& InUserId = FString(), const FAuthTokenGoogle& InAuthToken = FAuthTokenGoogle())
		: FUserOnlineAccountGoogleCommon(InUserId, InAuthToken)
	{
	}

	virtual ~FUserOnlineAccountGoogle()
	{
	}
};

/**
 * Google service implementation of the online identity interface
 */
class FOnlineIdentityGoogle :
	public FOnlineIdentityGoogleCommon
{

public:
	// IOnlineIdentity
	
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;

	// FOnlineIdentityGoogle

	FOnlineIdentityGoogle(FOnlineSubsystemGoogle* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityGoogle();

	bool Init();

PACKAGE_SCOPE:

	/** Checks config to know if we should request a server auth code */
	static bool ShouldRequestServerAuthCode();
	
	/**
	 * Login with an existing token
	 *
	 * @param LocalUserNum id of the local user initiating the request
	 * @param InToken exchange or refresh token already received from Google
	 * @param InCompletionDelegate delegate to fire when operation completes
	 */
	void Login(int32 LocalUserNum, const FAuthTokenGoogle& InToken, const FOnLoginCompleteDelegate& InCompletionDelegate);

private:
	void OnSignInComplete(const FGoogleSignInData& InSignInData);
	void OnSignOutComplete(const FGoogleSignOutData& InSignOutData);

	void OnAccessTokenLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UniqueId, const FString& Error);
	void OnLoginAttemptComplete(int32 LocalUserNum, const FString& ErrorStr);

	/** ObjC helper for access to SDK methods and callbacks */
	FGoogleHelper* GoogleHelper;

	/** Config based list of permission scopes to use when logging in */
	TArray<FString> ScopeFields;
	
	/** Is it allowed to try to refresh the tokens instead of full a login? */
	bool bAllowSilentSignIn;

	/** Delegate holder for all internal related login callbacks */
	DECLARE_DELEGATE_TwoParams(FOnInternalLoginComplete, EGoogleLoginResponse /*InLoginResponse*/, const FAuthTokenGoogle& /*AccessToken*/);
	FOnInternalLoginComplete LoginCompletionDelegate;
	/** Delegate holder for all internal related logout callbacks */
	DECLARE_DELEGATE_OneParam(FOnInternalLogoutComplete, EGoogleLoginResponse /*InLoginResponse*/);
	FOnInternalLogoutComplete LogoutCompletionDelegate;
};

typedef TSharedPtr<FOnlineIdentityGoogle, ESPMode::ThreadSafe> FOnlineIdentityGooglePtr;
