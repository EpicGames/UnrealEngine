// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemGoogle.h"
#include "OnlineSubsystemGooglePrivate.h"

#include "IOS/IOSAppDelegate.h"

#include "OnlineIdentityGoogle.h"
#include "OnlineExternalUIInterfaceGoogleIOS.h"

THIRD_PARTY_INCLUDES_START
#import <GoogleSignIn/GoogleSignIn.h>
THIRD_PARTY_INCLUDES_END

static void OnGoogleOpenURL(UIApplication* application, NSURL* url, NSString* sourceApplication, id annotation)
{
	bool bResult = [[GIDSignIn sharedInstance] handleURL:url];
	UE_LOG_ONLINE(Display, TEXT("OnGoogleOpenURL %s %d"), *FString(url.absoluteString), bResult);
}

FOnlineSubsystemGoogle::FOnlineSubsystemGoogle(FName InInstanceName)
	: FOnlineSubsystemGoogleCommon(InInstanceName)
{
	bPlatformRequiresClientId = true;
	bPlatformAllowsClientIdOverride = false;	
	bPlatformRequiresServerClientId = FOnlineIdentityGoogle::ShouldRequestOfflineAccess();
}

FOnlineSubsystemGoogle::~FOnlineSubsystemGoogle()
{
}

bool FOnlineSubsystemGoogle::Init()
{
	FIOSCoreDelegates::OnOpenURL.AddStatic(&OnGoogleOpenURL);

	if (FOnlineSubsystemGoogleCommon::Init())
	{
		FOnlineIdentityGooglePtr TempPtr = MakeShareable(new FOnlineIdentityGoogle(this));
		if (TempPtr->Init())
		{
			GoogleIdentity = TempPtr;
		}
		
		GoogleExternalUI = MakeShareable(new FOnlineExternalUIGoogleIOS(this));
		return GoogleIdentity.IsValid();
	}

	return false;
}

bool FOnlineSubsystemGoogle::Shutdown()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemGoogle::Shutdown()"));
	return FOnlineSubsystemGoogleCommon::Shutdown();
}
