// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAccountGoogleCommon.h"
#include "OnlineSubsystemGooglePrivate.h"

bool FUserOnlineAccountGoogleCommon::Parse(const FAuthTokenGoogle& InAuthToken, const FString& InJsonStr)
{
	bool bResult = false;
	if (InAuthToken.IsValid())
	{
		if (!InJsonStr.IsEmpty())
		{
			TSharedPtr<FJsonObject> JsonUser;
			TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(InJsonStr);

			if (FJsonSerializer::Deserialize(JsonReader, JsonUser) &&
				JsonUser.IsValid())
			{
				bResult = Parse(InAuthToken, JsonUser);
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountGoogleCommon: Can't deserialize payload=%s"), *InJsonStr);
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountGoogleCommon: Empty Json string"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountGoogleCommon: Invalid auth token"));
	}

	return bResult;
}

bool FUserOnlineAccountGoogleCommon::Parse(const FAuthTokenGoogle& InAuthToken, TSharedPtr<FJsonObject> InJsonObject)
{
	bool bResult = false;
	if (InAuthToken.IsValid())
	{
		if (InJsonObject.IsValid())
		{
			if (FromJson(InJsonObject))
			{
				if (!UserId.IsEmpty())
				{
					UserIdPtr = FUniqueNetIdGoogle::Create(UserId);

					// update the access token
					AuthToken = InAuthToken;

					bResult = true;
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountGoogleCommon: Missing user id in json object"));
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountGoogleCommon: Invalid json object"));
			}
		}	
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountGoogleCommon: Invalid json object pointer"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountGoogleCommon: Invalid auth token"));
	}

	return bResult;
}

FUniqueNetIdRef FUserOnlineAccountGoogleCommon::GetUserId() const
{
	return UserIdPtr;
}

FString FUserOnlineAccountGoogleCommon::GetRealName() const
{
	return RealName;
}

FString FUserOnlineAccountGoogleCommon::GetDisplayName(const FString& Platform) const
{
	return RealName;
}

bool FUserOnlineAccountGoogleCommon::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return GetAccountData(AttrName, OutAttrValue);
}

bool FUserOnlineAccountGoogleCommon::SetUserAttribute(const FString& AttrName, const FString& AttrValue)
{
	return SetAccountData(AttrName, AttrValue);
}

FString FUserOnlineAccountGoogleCommon::GetAccessToken() const
{
	return AuthToken.AccessToken;
}

bool FUserOnlineAccountGoogleCommon::GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return AuthToken.GetAuthData(AttrName, OutAttrValue);
}

