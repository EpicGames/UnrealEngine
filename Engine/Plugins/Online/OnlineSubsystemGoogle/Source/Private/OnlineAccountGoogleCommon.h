// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "OnlineSubsystemGoogleTypes.h"
#include "Serialization/JsonSerializerMacros.h"
#include "OnlineSubsystemGooglePackage.h"
#include "Interfaces/IHttpRequest.h"

/**
 * Info associated with an user account generated by this online service
 */
class FUserOnlineAccountGoogleCommon : 
	public FUserOnlineAccount,
	public FJsonSerializable
{
public:

	// FOnlineUser
	
	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;
	virtual bool SetUserAttribute(const FString& AttrName, const FString& AttrValue) override;

	// FUserOnlineAccount

	virtual FString GetAccessToken() const override;
	virtual bool GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FUserOnlineAccountGoogleCommon

	explicit FUserOnlineAccountGoogleCommon(const FString& InUserId = FString(), const FAuthTokenGoogle& InAuthToken = FAuthTokenGoogle())
		: UserIdPtr(FUniqueNetIdGoogle::Create(InUserId))
		, UserId(InUserId)
		, AuthToken(InAuthToken)
	{
	}

	virtual ~FUserOnlineAccountGoogleCommon()
	{
	}

	/**
	 * Parse Json profile request data into the user account
	 *
	 * @param InAuthToken previously associated auth token with this Json response
	 * @param InJsonStr a json payload from a /me request
	 */
	bool Parse(const FAuthTokenGoogle& InAuthToken, const FString& InJsonStr);

	/**
	 * Parse Json profile request data into the user account
	 *
	 * @param InAuthToken previously associated auth token with this Json response
	 * @param InJsonObject a json payload from a /me request
	 */
	bool Parse(const FAuthTokenGoogle& InAuthToken, TSharedPtr<FJsonObject> InJsonObject);

protected:

	/**
	 * Get account data attribute
	 *
	 * @param Key account data entry key
	 * @param OutVal [out] value that was found
	 *
	 * @return true if entry was found
	 */
	inline bool GetAccountData(const FString& Key, FString& OutVal) const
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal != NULL)
		{
			OutVal = *FoundVal;
			return true;
		}
		return false;
	}

	/**
	 * Set account data attribute
	 *
	 * @param Key account data entry key
	 * @param Val value that we want to set
	 *
	 * @return true if entry changed
	 */
	inline bool SetAccountData(const FString& Key, const FString& Val)
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal == NULL || *FoundVal != Val)
		{
			AccountData.Add(Key, Val);
			return true;
		}
		return false;
	}

	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserIdPtr;
	/** Id associated with the user account provided by the online service during registration */
	FString UserId;
	/** Real name */
	FString RealName;
	/** First name */
	FString FirstName;
	/** Last name */
	FString LastName;
	/** Token which is provided to user once authenticated by the online service */
	FAuthTokenGoogle AuthToken;
	/** Any addition account data associated with the user */
	FJsonSerializableKeyValueMap AccountData;

private:

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("id", UserId); //v2
		JSON_SERIALIZE("sub", UserId); //v3
		JSON_SERIALIZE("given_name", FirstName);
		JSON_SERIALIZE("family_name", LastName);
		JSON_SERIALIZE("name", RealName);
		JSON_SERIALIZE_SIMPLECOPY(AccountData);
	END_JSON_SERIALIZER

	/** Allow the Google identity to fill in our private members from it's callbacks */
	friend class FOnlineIdentityGoogle;
};
