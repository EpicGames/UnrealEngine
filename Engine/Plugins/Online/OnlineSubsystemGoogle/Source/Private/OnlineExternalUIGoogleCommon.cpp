// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineExternalUIGoogleCommon.h"
#include "OnlineSubsystemGoogle.h"
#if USES_RESTFUL_GOOGLE
#include "OnlineIdentityGoogleRest.h"
#else
#include "OnlineIdentityGoogle.h"
#endif

bool FOnlineExternalUIGoogleCommon::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	bool bStarted = false;

	GoogleSubsystem->ExecuteNextTick([ControllerIndex, Delegate]()
	{
		Delegate.ExecuteIfBound(nullptr, ControllerIndex, FOnlineError(EOnlineErrorResult::Unknown));
	});

	return bStarted;
}

bool FOnlineExternalUIGoogleCommon::ShowFriendsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUIGoogleCommon::ShowInviteUI(int32 LocalUserNum, FName SessionName)
{
	return false;
}

bool FOnlineExternalUIGoogleCommon::ShowAchievementsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUIGoogleCommon::ShowLeaderboardUI( const FString& LeaderboardName )
{
	return false;
}

bool FOnlineExternalUIGoogleCommon::ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIGoogleCommon::CloseWebURL()
{
	return false;
}

bool FOnlineExternalUIGoogleCommon::ShowAccountUpgradeUI(const FUniqueNetId& UniqueId)
{
	return false;
}

bool FOnlineExternalUIGoogleCommon::ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIGoogleCommon::ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIGoogleCommon::ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate)
{
	return false;
}

