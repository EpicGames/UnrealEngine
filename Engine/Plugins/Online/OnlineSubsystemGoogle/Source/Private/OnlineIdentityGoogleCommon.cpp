// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityGoogleCommon.h"
#if USES_RESTFUL_GOOGLE
#include "OnlineIdentityGoogleRest.h"
#else
#include "OnlineIdentityGoogle.h"
#endif
#include "OnlineSubsystemGooglePrivate.h"
#include "OnlineSubsystemGoogleTypes.h"
#include "OnlineSubsystemGoogle.h"
#include "OnlineError.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Base64.h"

bool FJsonWebTokenGoogle::Parse(const FString& InJWTStr)
{
	bool bSuccess = false;

	TArray<FString> Tokens;
	InJWTStr.ParseIntoArray(Tokens, TEXT("."));
	if (Tokens.Num() == 3)
	{
		// Figure out if any Base64 padding adjustment is necessary
		static const TCHAR* const Padding = TEXT("==");
		int32 Padding1 = (4 - (Tokens[0].Len() % 4)) % 4;
		int32 Padding2 = (4 - (Tokens[1].Len() % 4)) % 4;
		int32 Padding3 = (4 - (Tokens[2].Len() % 4)) % 4;
		if (Padding1 < 3 && Padding2 < 3 && Padding3 < 3)
		{
			Tokens[0].AppendChars(Padding, Padding1);
			Tokens[1].AppendChars(Padding, Padding2);
			Tokens[2].AppendChars(Padding, Padding3);

			// Decode JWT header
			FString HeaderStr;
			if (FBase64::Decode(Tokens[0], HeaderStr))
			{
				// Parse header
				if (Header.FromJson(HeaderStr))
				{
					// Decode JWT payload
					FString PayloadStr;
					if (FBase64::Decode(Tokens[1], PayloadStr))
					{
						// Parse payload
						if (Payload.FromJson(PayloadStr))
						{
							// @TODO - Verify that the ID token is properly signed by the issuer.Google
							// issued tokens are signed using one of the certificates found at the URI specified in the jwks_uri field of the discovery document.

							//Verify that the value of iss in the ID token is Google issued
							static const FString Issuer1 = TEXT("https://accounts.google.com");
							static const FString Issuer2 = TEXT("accounts.google.com");
							if ((Payload.ISS == Issuer1) || (Payload.ISS == Issuer2))
							{
								// Verify that the value of aud in the ID token is equal to your app's client ID.
								FOnlineSubsystemGoogle* GoogleSubsystem = static_cast<FOnlineSubsystemGoogle*>(IOnlineSubsystem::Get(GOOGLE_SUBSYSTEM));
								if (ensure(GoogleSubsystem))
								{
									if (Payload.Aud == GoogleSubsystem->GetAppId() ||
										Payload.Aud == GoogleSubsystem->GetServerClientId())
									{
										//https://www.codescience.com/blog/2016/oauth2-server-to-server-authentication-from-salesforce-to-google-apis
										// exp	Required	The expiration time of the assertion, specified as seconds since 00:00:00 UTC, January 1, 1970. This value has a maximum of 1 hour after the issued time.
										// iat	Required	The time the assertion was issued, specified as seconds since 00:00:00 UTC, January 1, 1970.

										//Verify that the expiry time(exp) of the ID token has not passed.
										FDateTime ExpiryTime = FDateTime::FromUnixTimestamp(Payload.EXP);
										FDateTime IssueTime = FDateTime::FromUnixTimestamp(Payload.IAT);
										if ((ExpiryTime - IssueTime) <= FTimespan(ETimespan::TicksPerHour) && ExpiryTime > FDateTime::UtcNow())
										{
											bSuccess = true;
#if 0
											TArray<uint8> Signature;
											if (FBase64::Decode(Tokens[2], Signature))
											{
												bSuccess = true;
											}
#endif
										}
										else
										{
											UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google auth: Expiry Time inconsistency"));
											UE_LOG_ONLINE_IDENTITY(Warning, TEXT("	Expiry: %s"), *ExpiryTime.ToString());
											UE_LOG_ONLINE_IDENTITY(Warning, TEXT("	Issue: %s"), *IssueTime.ToString());
										}
									}
									else
									{
										UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google auth: Audience inconsistency"));
										UE_LOG_ONLINE_IDENTITY(Warning, TEXT("	Payload: %s"), *Payload.Aud);
										UE_LOG_ONLINE_IDENTITY(Warning, TEXT("	ClientId: %s"), *GoogleSubsystem->GetAppId());
										UE_LOG_ONLINE_IDENTITY(Warning, TEXT("	ServerClientId: %s"), *GoogleSubsystem->GetServerClientId());
									}
								}
								else
								{
									UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google auth: missing OSS"));
								}
							}
							else
							{
								UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google auth: Issuer inconsistency"));
								UE_LOG_ONLINE_IDENTITY(Warning, TEXT("	ISS: %s"), *Payload.ISS);
							}
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google auth: Payload data inconsistency"));
						}
					}
					else
					{
						UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google auth: Payload format inconsistency"));
					}
				}
				else
				{
					UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google auth: Header data inconsistency"));
				}
			}
			else
			{
				UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google auth: Header format inconsistency"));
			}
		}
		else
		{
			UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google auth: JWT format inconsistency"));
		}
	}

	return bSuccess;
}

void FAuthTokenGoogle::AddAuthAttributes(const TSharedPtr<FJsonObject>& JsonUser)
{
	for (auto It = JsonUser->Values.CreateConstIterator(); It; ++It)
	{
		if (It.Value().IsValid())
		{
			if (It.Value()->Type == EJson::String)
			{
				AuthData.Add(FString(It.Key()), It.Value()->AsString());
			}
			else if (It.Value()->Type == EJson::Boolean)
			{
				AuthData.Add(FString(It.Key()), It.Value()->AsBool() ? TEXT("true") : TEXT("false"));
			}
			else if (It.Value()->Type == EJson::Number)
			{
				AuthData.Add(FString(It.Key()), FString::Printf(TEXT("%f"), (double)It.Value()->AsNumber()));
			}
		}
	}
}

bool FAuthTokenGoogle::Parse(const FString& InJsonStr, const FAuthTokenGoogle& InOldAuthToken)
{
	bool bSuccess = false;
	if ((InOldAuthToken.AuthType == EGoogleAuthTokenType::RefreshToken) && Parse(InJsonStr))
	{
		RefreshToken = InOldAuthToken.RefreshToken;
		AuthData.Add(TEXT("refresh_token"), InOldAuthToken.RefreshToken);
		bSuccess = true;
	}

	return bSuccess;
}

bool FAuthTokenGoogle::Parse(const FString& InJsonStr)
{
	bool bSuccess = false;

	if (!InJsonStr.IsEmpty())
	{
		TSharedPtr<FJsonObject> JsonAuth;
		TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(InJsonStr);

		if (FJsonSerializer::Deserialize(JsonReader, JsonAuth) &&
			JsonAuth.IsValid())
		{
			bSuccess = Parse(JsonAuth);
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FAuthTokenGoogle: Empty Json string"));
	}

	return bSuccess;
}

bool FAuthTokenGoogle::Parse(TSharedPtr<FJsonObject> InJsonObject)
{
	bool bSuccess = false;

	if (InJsonObject.IsValid())
	{
		if (FromJson(InJsonObject))
		{
			if (!AccessToken.IsEmpty())
			{
				if (IdTokenJWT.Parse(IdToken))
				{
					AddAuthAttributes(InJsonObject);
					AuthType = EGoogleAuthTokenType::AccessToken;
					ExpiresInUTC = FDateTime::UtcNow() + FTimespan(ExpiresIn * ETimespan::TicksPerSecond);
					bSuccess = true;
				}
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FAuthTokenGoogle: Invalid Json pointer"));
	}

	return bSuccess;
}

bool FOnlineIdentityGoogleCommon::ShouldRequestOfflineAccess()
{
	bool bRequestOfflineAccess = false;
	GConfig->GetBool(TEXT("OnlineSubsystemGoogle.OnlineIdentityGoogle"), TEXT("bRequestOfflineAccess"), bRequestOfflineAccess, GEngineIni);
	return bRequestOfflineAccess;
}

FOnlineIdentityGoogleCommon::FOnlineIdentityGoogleCommon(FOnlineSubsystemGoogle* InSubsystem)
	: GoogleSubsystem(InSubsystem)
{
}

TSharedPtr<FUserOnlineAccount> FOnlineIdentityGoogleCommon::GetUserAccount(const FUniqueNetId& UserId) const
{
	TSharedPtr<FUserOnlineAccount> Result;

	const TSharedRef<FUserOnlineAccountGoogleCommon>* FoundUserAccount = UserAccounts.Find(UserId.ToString());
	if (FoundUserAccount != nullptr)
	{
		Result = *FoundUserAccount;
	}

	return Result;
}

TArray<TSharedPtr<FUserOnlineAccount> > FOnlineIdentityGoogleCommon::GetAllUserAccounts() const
{
	TArray<TSharedPtr<FUserOnlineAccount> > Result;
	
	for (FUserOnlineAccountGoogleMap::TConstIterator It(UserAccounts); It; ++It)
	{
		Result.Add(It.Value());
	}

	return Result;
}

FUniqueNetIdPtr FOnlineIdentityGoogleCommon::GetUniquePlayerId(int32 LocalUserNum) const
{
	const FUniqueNetIdPtr* FoundId = UserIds.Find(LocalUserNum);
	if (FoundId != nullptr)
	{
		return *FoundId;
	}
	return nullptr;
}


void FOnlineIdentityGoogleCommon::RetrieveDiscoveryDocument(PendingLoginRequestCb&& LoginCb)
{
	if (!Endpoints.IsValid())
	{
		static const FString DiscoveryURL = TEXT("https://accounts.google.com/.well-known/openid-configuration");
		// kick off http request to get the discovery document
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

		HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOnlineIdentityGoogleCommon::DiscoveryRequest_HttpRequestComplete, LoginCb);
		HttpRequest->SetURL(DiscoveryURL);
		HttpRequest->SetVerb(TEXT("GET"));
		HttpRequest->ProcessRequest();
	}
	else
	{
		LoginCb(true);
	}
}

void FOnlineIdentityGoogleCommon::DiscoveryRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, PendingLoginRequestCb LoginCb)
{
	if (bSucceeded &&
		HttpResponse.IsValid())
	{
		FString ResponseStr = HttpResponse->GetContentAsString();
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
			UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("Discovery request complete. url=%s code=%d response=%s"),
				*HttpRequest->GetURL(), HttpResponse->GetResponseCode(), *ResponseStr);
			if (!Endpoints.Parse(ResponseStr))
			{
				UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Failed to parse Google discovery endpoint"));
			}
		}
		else
		{
			UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Bad response from Google discovery endpoint"));
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Google discovery endpoint failure"));
	}

	LoginCb(Endpoints.IsValid());
}

void FOnlineIdentityGoogleCommon::ProfileRequest(int32 LocalUserNum, const FAuthTokenGoogle& InAuthToken, FOnProfileRequestComplete& InCompletionDelegate)
{
	FString ErrorStr;
	bool bStarted = false;
	if (LocalUserNum >= 0 && LocalUserNum < MAX_LOCAL_PLAYERS)
	{
		if (Endpoints.IsValid() && !Endpoints.UserInfoEndpoint.IsEmpty())
		{
			if (InAuthToken.IsValid())
			{
				check(InAuthToken.AuthType == EGoogleAuthTokenType::AccessToken);
				bStarted = true;

				// kick off http request to get user info with the access token
				TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

				const FString BearerToken = FString::Printf(TEXT("Bearer %s"), *InAuthToken.AccessToken);

				HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOnlineIdentityGoogleCommon::MeUser_HttpRequestComplete, LocalUserNum, InAuthToken, InCompletionDelegate);
				HttpRequest->SetURL(Endpoints.UserInfoEndpoint);
				HttpRequest->SetHeader(TEXT("Authorization"), BearerToken);
				HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
				HttpRequest->SetVerb(TEXT("GET"));
				HttpRequest->ProcessRequest();
			}
			else
			{
				ErrorStr = TEXT("Invalid access token specified");
			}
		}
		else
		{
			ErrorStr = TEXT("No MeURL specified in DefaultEngine.ini");
		}
	}
	else
	{
		ErrorStr = TEXT("Invalid local user num");
	}

	if (!bStarted)
	{
		InCompletionDelegate.ExecuteIfBound(LocalUserNum, false, ErrorStr);
	}
}

void FOnlineIdentityGoogleCommon::MeUser_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, int32 InLocalUserNum, FAuthTokenGoogle InAuthToken, FOnProfileRequestComplete InCompletionDelegate)
{
	bool bResult = false;
	FString ResponseStr, ErrorStr;
	
	if (bSucceeded &&
		HttpResponse.IsValid())
	{
		ResponseStr = HttpResponse->GetContentAsString();
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
			UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("RegisterUser request complete. url=%s code=%d response=%s"),
				*HttpRequest->GetURL(), HttpResponse->GetResponseCode(), *ResponseStr);

			TSharedRef<FUserOnlineAccountGoogle> User = MakeShared<FUserOnlineAccountGoogle>();
			if (User->Parse(InAuthToken, ResponseStr))
			{
				// update/add cached entry for user
				UserAccounts.Add(User->GetUserId()->ToString(), User);
				// keep track of user ids for local users
				UserIds.Add(InLocalUserNum, User->GetUserId());

				bResult = true;
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("Error parsing login. payload=%s"),
					*ResponseStr);
			}
		}
		else
		{
			FErrorGoogle Error;
			if (Error.FromJson(ResponseStr) && !Error.Error_Description.IsEmpty())
			{
				ErrorStr = Error.Error_Description;
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("Failed to parse Google error %s"), *ResponseStr);
			}
		}
	}
	else
	{
		ErrorStr = TEXT("No response");
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("RegisterUser request failed. %s"), *ErrorStr);
	}

	InCompletionDelegate.ExecuteIfBound(InLocalUserNum, bResult, ErrorStr);
}

FUniqueNetIdPtr FOnlineIdentityGoogleCommon::CreateUniquePlayerId(uint8* Bytes, int32 Size)
{
	if (Bytes != nullptr && Size > 0)
	{
		FString StrId = FString::ConstructFromPtrSize((TCHAR*)Bytes, Size);
		return FUniqueNetIdGoogle::Create(StrId);
	}
	return nullptr;
}

FUniqueNetIdPtr FOnlineIdentityGoogleCommon::CreateUniquePlayerId(const FString& Str)
{
	return FUniqueNetIdGoogle::Create(Str);
}

bool FOnlineIdentityGoogleCommon::AutoLogin(int32 LocalUserNum)
{
	return false;
}

ELoginStatus::Type FOnlineIdentityGoogleCommon::GetLoginStatus(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		return GetLoginStatus(*UserId);
	}
	return ELoginStatus::NotLoggedIn;
}

ELoginStatus::Type FOnlineIdentityGoogleCommon::GetLoginStatus(const FUniqueNetId& UserId) const
{
	TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(UserId);
	if (UserAccount.IsValid() &&
		UserAccount->GetUserId()->IsValid() &&
		(!bAccessTokenAvailableToPlatform || !UserAccount->GetAccessToken().IsEmpty()))
	{
		return ELoginStatus::LoggedIn;
	}
	return ELoginStatus::NotLoggedIn;
}

FString FOnlineIdentityGoogleCommon::GetPlayerNickname(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		return GetPlayerNickname(*UserId);
	}
	return TEXT("");
}

FString FOnlineIdentityGoogleCommon::GetPlayerNickname(const FUniqueNetId& UserId) const
{
	const TSharedRef<FUserOnlineAccountGoogleCommon>* FoundUserAccount = UserAccounts.Find(UserId.ToString());
	if (FoundUserAccount != nullptr)
	{
		const TSharedRef<FUserOnlineAccountGoogleCommon>& UserAccount = *FoundUserAccount;
		return UserAccount->GetRealName();
	}
	return TEXT("");
}

FString FOnlineIdentityGoogleCommon::GetAuthToken(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(*UserId);
		if (UserAccount.IsValid())
		{
			return UserAccount->GetAccessToken();
		}
	}
	return FString();
}

void FOnlineIdentityGoogleCommon::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI)
{
	Delegate.ExecuteIfBound(UserId, Privilege, (uint32)EPrivilegeResults::NoFailures);
}	

FPlatformUserId FOnlineIdentityGoogleCommon::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const
{
	for (int i = 0; i < MAX_LOCAL_PLAYERS; ++i)
	{
		auto CurrentUniqueId = GetUniquePlayerId(i);
		if (CurrentUniqueId.IsValid() && (*CurrentUniqueId == UniqueNetId))
		{
			return GetPlatformUserIdFromLocalUserNum(i);
		}
	}

	return PLATFORMUSERID_NONE;
}

FString FOnlineIdentityGoogleCommon::GetAuthType() const
{
	return AUTH_TYPE_GOOGLE;
}

void FOnlineIdentityGoogleCommon::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityGoogleCommon::RevokeAuthToken not implemented"));
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	GoogleSubsystem->ExecuteNextTick([UserIdRef, Delegate]()
	{
		Delegate.ExecuteIfBound(*UserIdRef, FOnlineError(FString(TEXT("RevokeAuthToken not implemented"))));
	});
}
