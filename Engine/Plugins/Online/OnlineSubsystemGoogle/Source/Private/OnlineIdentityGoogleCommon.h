// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystemGoogleTypes.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Interfaces/IHttpRequest.h"
#include "OnlineSubsystemGooglePackage.h"

// Google scope fields: https://developers.google.com/identity/protocols/googlescopes
// email profile
// https://www.googleapis.com/auth/userinfo.email
// https://www.googleapis.com/auth/userinfo.profile
#define GOOGLE_PERM_PUBLIC_PROFILE "https://www.googleapis.com/auth/userinfo.profile"

class FOnlineSubsystemGoogle;
class FUserOnlineAccountGoogleCommon;

#define AUTH_TYPE_GOOGLE TEXT("google")
#define GOOGLE_AUTH_EXPIRED_CREDS TEXT("com.epicgames.google.oauth.expiredcreds")

/**
 * Delegate fired after a Google profile request has been completed
 *
 * @param LocalUserNum the controller number of the associated user
 * @param bWasSuccessful was the request successful
 * @param ErrorStr error associated with the request
 */
DECLARE_DELEGATE_ThreeParams(FOnProfileRequestComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FString& /*ErrorStr*/);

/** Mapping from user id to internal online account info (only one per user) */
typedef TMap<FString, TSharedRef<FUserOnlineAccountGoogleCommon> > FUserOnlineAccountGoogleMap;

/**
 * Google service implementation of the online identity interface
 */
class FOnlineIdentityGoogleCommon :
	public IOnlineIdentity
{

protected:

	/** Parent subsystem */
	FOnlineSubsystemGoogle* GoogleSubsystem;
	/** Endpoint configurations retrieved from Google discovery service */
	FGoogleOpenIDConfiguration Endpoints;
	/** Users that have been registered/authenticated */
	FUserOnlineAccountGoogleMap UserAccounts;
	/** Ids mapped to locally registered users */
	TMap<int32, FUniqueNetIdPtr > UserIds;
	/** Platform can know the client access token */
	bool bAccessTokenAvailableToPlatform = true;

	typedef TFunction<void(bool)> PendingLoginRequestCb;

public:

	// IOnlineIdentity
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI=EShowPrivilegeResolveUI::Default) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const override;
	virtual FString GetAuthType() const override;
	virtual void RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;

	// FOnlineIdentityGoogle

	FOnlineIdentityGoogleCommon(FOnlineSubsystemGoogle* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityGoogleCommon()
	{
	}

PACKAGE_SCOPE:

	/** Checks config to know if we should request offline access (Server Auth code on mobile or refresh token on REST implementation)*/
	static bool ShouldRequestOfflineAccess();

	/**
	 * Retrieve the profile for a given user and access token
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param AccessToken associated access token to make the request
	 * @param InCompletionDelegate delegate to fire when request is completed
	 */
	void ProfileRequest(int32 LocalUserNum, const FAuthTokenGoogle& InAuthToken, FOnProfileRequestComplete& InCompletionDelegate);


	/**
	 * Retrieve auth endpoints from Google discovery service
	 *
	 * @param LoginCb the login function to call after this request completes
	 */
	void RetrieveDiscoveryDocument(PendingLoginRequestCb&& LoginCb);

protected:
	/**
	 * Delegate fired when the discover service request has completed
	 */
	virtual void DiscoveryRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, PendingLoginRequestCb LoginCb);

private:

	/**
	 * Delegate called when a user /me request from Google is complete
	 */
	void MeUser_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, int32 InLocalUserNum, FAuthTokenGoogle InAuthToken, FOnProfileRequestComplete InCompletionDelegate);
};

typedef TSharedPtr<FOnlineIdentityGoogleCommon, ESPMode::ThreadSafe> FOnlineIdentityGoogleCommonPtr;
