// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemGoogleCommon.h"
#include "OnlineSubsystemGooglePrivate.h"
#include "OnlineIdentityGoogleCommon.h"
#include "OnlineExternalUIGoogleCommon.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "Stats/Stats.h"

#define GOOGLE_CLIENTAUTH_ID TEXT("ClientId")
#define GOOGLE_SERVERAUTH_ID TEXT("ServerClientId")

FOnlineSubsystemGoogleCommon::FOnlineSubsystemGoogleCommon(FName InInstanceName)
	: FOnlineSubsystemImpl(GOOGLE_SUBSYSTEM, InInstanceName)
	, bPlatformRequiresClientId(false)
	, bPlatformAllowsClientIdOverride(false)
	, bPlatformRequiresServerClientId(false)
{
}

FOnlineSubsystemGoogleCommon::~FOnlineSubsystemGoogleCommon()
{
}

bool FOnlineSubsystemGoogleCommon::Init()
{
	static FString ConfigSection(TEXT("OnlineSubsystemGoogle"));
	if (bPlatformRequiresClientId)
	{
		if (!GConfig->GetString(*ConfigSection, GOOGLE_CLIENTAUTH_ID, ClientId, GEngineIni))
		{
			UE_LOG_ONLINE(Warning, TEXT("Missing ClientId= in [%s] of DefaultEngine.ini"), *ConfigSection);
		}
	}
	
	if (bPlatformRequiresServerClientId)
	{
		if (!GConfig->GetString(*ConfigSection, GOOGLE_SERVERAUTH_ID, ServerClientId, GEngineIni))
		{
			UE_LOG_ONLINE(Warning, TEXT("Missing ServerClientId= in [%s] of DefaultEngine.ini"), *ConfigSection);
		}
	}
	
	FString ConfigOverride;
	FGoogleAuthConfig NewConfig;
	if (GetConfigurationDelegate().ExecuteIfBound(ConfigOverride, NewConfig))
	{
		if (!NewConfig.Backend.IsEmpty())
		{
			FString IniSection = ConfigSection + FString(TEXT(" ")) + NewConfig.Backend;
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FOnlineSubsystemGoogleCommon::Init IniSection:%s"), *IniSection);

			FString NewClientAuthId;
			if (GConfig->GetString(*IniSection, GOOGLE_CLIENTAUTH_ID, NewClientAuthId, GEngineIni) && !NewClientAuthId.IsEmpty())
			{
				if (bPlatformRequiresClientId && bPlatformAllowsClientIdOverride)
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FOnlineSubsystemGoogleCommon::Init ClientId:%s"), *NewClientAuthId);
					ClientId = NewClientAuthId;
				}
				else
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FOnlineSubsystemGoogleCommon::Init Platform does not support ClientId override"), *NewClientAuthId);
				}
			}

			FString NewServerAuthId;
			if (GConfig->GetString(*IniSection, GOOGLE_SERVERAUTH_ID, NewServerAuthId, GEngineIni) && !NewServerAuthId.IsEmpty())
			{
				if (bPlatformRequiresServerClientId)
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FOnlineSubsystemGoogleCommon::Init ServerClientId:%s"), *NewServerAuthId);
					ServerClientId = NewServerAuthId;
				}
				else
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FOnlineSubsystemGoogleCommon::Init Platform does not support ServerClientId override"), *NewServerAuthId);
				}
			}
		}
	}
	else
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("GetConfigurationDelegate was not bound!"));
	}

	return true;
}

bool FOnlineSubsystemGoogleCommon::Shutdown()
{
	UE_LOG_ONLINE(Display, TEXT("FOnlineSubsystemGoogleCommon::Shutdown()"));

	FOnlineSubsystemImpl::Shutdown();

#define DESTRUCT_INTERFACE(Interface) \
	if (Interface.IsValid()) \
	{ \
		ensure(Interface.IsUnique()); \
		Interface = nullptr; \
	}

	DESTRUCT_INTERFACE(GoogleExternalUI);
	DESTRUCT_INTERFACE(GoogleIdentity);

#undef DESTRUCT_INTERFACE

	return true;
}

FOnlineSubsystemGoogleCommon::FGoogleConfigurationDelegate& FOnlineSubsystemGoogleCommon::GetConfigurationDelegate()
{
	static FGoogleConfigurationDelegate Delegate;
	return Delegate;
}

bool FOnlineSubsystemGoogleCommon::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemGoogleCommon_Tick);

	if (!FOnlineSubsystemImpl::Tick(DeltaTime))
	{
		return false;
	}

	return true;
}

FString FOnlineSubsystemGoogleCommon::GetAppId() const
{
	return ClientId;
}

bool FOnlineSubsystemGoogleCommon::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}
	return false;
}

IOnlineSessionPtr FOnlineSubsystemGoogleCommon::GetSessionInterface() const
{
	return nullptr;
}

IOnlineFriendsPtr FOnlineSubsystemGoogleCommon::GetFriendsInterface() const
{
	return nullptr;
}

IOnlinePartyPtr FOnlineSubsystemGoogleCommon::GetPartyInterface() const
{
	return nullptr;
}

IOnlineGroupsPtr FOnlineSubsystemGoogleCommon::GetGroupsInterface() const
{
	return nullptr;
}

IOnlineSharedCloudPtr FOnlineSubsystemGoogleCommon::GetSharedCloudInterface() const
{
	return nullptr;
}

IOnlineUserCloudPtr FOnlineSubsystemGoogleCommon::GetUserCloudInterface() const
{
	return nullptr;
}

IOnlineLeaderboardsPtr FOnlineSubsystemGoogleCommon::GetLeaderboardsInterface() const
{
	return nullptr;
}

IOnlineVoicePtr FOnlineSubsystemGoogleCommon::GetVoiceInterface() const
{
	return nullptr;
}

IOnlineExternalUIPtr FOnlineSubsystemGoogleCommon::GetExternalUIInterface() const	
{
	return GoogleExternalUI;
}

IOnlineTimePtr FOnlineSubsystemGoogleCommon::GetTimeInterface() const
{
	return nullptr;
}

IOnlineIdentityPtr FOnlineSubsystemGoogleCommon::GetIdentityInterface() const
{
	return GoogleIdentity;
}

IOnlineTitleFilePtr FOnlineSubsystemGoogleCommon::GetTitleFileInterface() const
{
	return nullptr;
}

IOnlineEntitlementsPtr FOnlineSubsystemGoogleCommon::GetEntitlementsInterface() const
{
	return nullptr;
}

IOnlineEventsPtr FOnlineSubsystemGoogleCommon::GetEventsInterface() const
{
	return nullptr;
}

IOnlineAchievementsPtr FOnlineSubsystemGoogleCommon::GetAchievementsInterface() const
{
	return nullptr;
}

IOnlineSharingPtr FOnlineSubsystemGoogleCommon::GetSharingInterface() const
{
	return nullptr;
}

IOnlineUserPtr FOnlineSubsystemGoogleCommon::GetUserInterface() const
{
	return nullptr;
}

IOnlineMessagePtr FOnlineSubsystemGoogleCommon::GetMessageInterface() const
{
	return nullptr;
}

IOnlinePresencePtr FOnlineSubsystemGoogleCommon::GetPresenceInterface() const
{
	return nullptr;
}

IOnlineChatPtr FOnlineSubsystemGoogleCommon::GetChatInterface() const
{
	return nullptr;
}

IOnlineStatsPtr FOnlineSubsystemGoogleCommon::GetStatsInterface() const
{
	return nullptr;
}

IOnlineTurnBasedPtr FOnlineSubsystemGoogleCommon::GetTurnBasedInterface() const
{
	return nullptr;
}

IOnlineTournamentPtr FOnlineSubsystemGoogleCommon::GetTournamentInterface() const
{
	return nullptr;
}

FText FOnlineSubsystemGoogleCommon::GetOnlineServiceName() const
{
	return NSLOCTEXT("OnlineSubsystemGoogleCommon", "OnlineServiceName", "Google");
}

