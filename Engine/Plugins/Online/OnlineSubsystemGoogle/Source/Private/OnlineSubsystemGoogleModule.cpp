// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemGoogleModule.h"
#include "OnlineSubsystemGooglePrivate.h"

IMPLEMENT_MODULE(FOnlineSubsystemGoogleModule, OnlineSubsystemGoogle);

/**
 * Class responsible for creating instance(s) of the subsystem
 */
class FOnlineFactoryGoogle : public IOnlineFactory
{
public:

	FOnlineFactoryGoogle() {}
	virtual ~FOnlineFactoryGoogle() {}

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName)
	{
		FOnlineSubsystemGooglePtr OnlineSub = MakeShared<FOnlineSubsystemGoogle, ESPMode::ThreadSafe>(InstanceName);
		if (OnlineSub->IsEnabled())
		{
			if(!OnlineSub->Init())
			{
				UE_LOG_ONLINE(Warning, TEXT("Google API failed to initialize!"));
				OnlineSub->Shutdown();
				OnlineSub = nullptr;
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Google API disabled!"));
			OnlineSub->Shutdown();
			OnlineSub = nullptr;
		}

		return OnlineSub;
	}
};

void FOnlineSubsystemGoogleModule::StartupModule()
{
	UE_LOG_ONLINE(Log, TEXT("Google Startup!"));

	GoogleFactory = new FOnlineFactoryGoogle();

	// Create and register our singleton factory with the main online subsystem for easy access
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.RegisterPlatformService(GOOGLE_SUBSYSTEM, GoogleFactory);
}

void FOnlineSubsystemGoogleModule::ShutdownModule()
{
	UE_LOG_ONLINE(Log, TEXT("Google Shutdown!"));

	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.UnregisterPlatformService(GOOGLE_SUBSYSTEM);

	delete GoogleFactory;
	GoogleFactory = nullptr;
}