// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#include "Http.h"

#include "OnlineSubsystemGoogle.h"
#include "OnlineSubsystemGoogleModule.h"

/** pre-pended to all Google logging */
#undef ONLINE_LOG_PREFIX
#define ONLINE_LOG_PREFIX TEXT("Google: ")
