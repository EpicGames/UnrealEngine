// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "Serialization/JsonSerializerMacros.h"
#include "OnlineSubsystemGooglePackage.h"

class FAuthTokenGoogle;
#if PLATFORM_IOS
@class GIDGoogleUser;
#endif

// from OnlineSubsystemTypes.h
TEMP_UNIQUENETIDSTRING_SUBCLASS(FUniqueNetIdGoogle, GOOGLE_SUBSYSTEM);

/**
 * Types of supported auth tokens
 */
enum class EGoogleAuthTokenType : uint8
{
	/** Simple single use token meant to be converted to an access token */
	ExchangeToken,
	/** Refresh token meant to be fully converted to an access token */
	RefreshToken,
	/** Allows for access to Google APIs using verified user account credentials */
	AccessToken
};

class FJsonWebTokenGoogle
{

public:

	/**
	 * Parse a Google JWT into its constituent parts
	 *
	 * @param InJWTStr JWT id token representations
	 *
	 * @return true if parsing was successful, false otherwise
	 */
	bool Parse(const FString& InJWTStr);

private:

	class FHeader : public FJsonSerializable
	{

	public:
		/** Algorithm used to sign the token */
		FString Alg;
		/** Key Id */
		FString Kid;
		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("alg", Alg);
			JSON_SERIALIZE("kid", Kid);
		END_JSON_SERIALIZER
	};

	class FPayload : public FJsonSerializable
	{

	public:

		/** Subscriber */
		FString Sub;
		/** User first name */
		FString FirstName;
		/** User last name */
		FString LastName;
		/** User full name */
		FString RealName;
		/** Issuer */
		FString ISS;
		/** Time of token grant */
		double IAT;
		/** Time of token expiration */
		double EXP;
		FString ATHash;
		/** Audience */
		FString Aud;
		/** Is the email address verified */
		bool bEmailVerified;
		FString AZP;
		/** User email address */
		FString Email;
		/** User profile picture */
		FString Picture;
		/** User locale */
		FString Locale;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("sub", Sub);
			JSON_SERIALIZE("given_name", FirstName);
			JSON_SERIALIZE("family_name", LastName);
			JSON_SERIALIZE("name", RealName);
			JSON_SERIALIZE("iss", ISS);
			JSON_SERIALIZE("iat", IAT);
			JSON_SERIALIZE("exp", EXP);
			JSON_SERIALIZE("at_hash", ATHash);
			JSON_SERIALIZE("aud", Aud);
			JSON_SERIALIZE("email_verified", bEmailVerified);
			JSON_SERIALIZE("azp", AZP);
			JSON_SERIALIZE("email", Email);
			JSON_SERIALIZE("picture", Picture);
			JSON_SERIALIZE("locale", Locale);
		END_JSON_SERIALIZER
	};

	/** JWT header */
	FHeader Header;
	/** JWT payload */
	FPayload Payload;
};

enum EGoogleExchangeToken : uint8 { GoogleExchangeToken };
enum EGoogleRefreshToken : uint8 { GoogleRefreshToken };

/**
 * Google auth token representation, both exchange and access tokens
 */
class FAuthTokenGoogle :
	public FJsonSerializable
{
public:

	FAuthTokenGoogle()
		: AuthType(EGoogleAuthTokenType::ExchangeToken)
		, ExpiresIn(0)
		, ExpiresInUTC(0)
	{
	}

	explicit FAuthTokenGoogle(const FString& InExchangeToken, EGoogleExchangeToken)
		: AuthType(EGoogleAuthTokenType::ExchangeToken)
		, AccessToken(InExchangeToken)
		, ExpiresIn(0)
		, ExpiresInUTC(0)
	{
	}

	explicit FAuthTokenGoogle(const FString& InRefreshToken, EGoogleRefreshToken)
		: AuthType(EGoogleAuthTokenType::RefreshToken)
		, ExpiresIn(0)
		, RefreshToken(InRefreshToken)
		, ExpiresInUTC(0)
	{
	}

	FAuthTokenGoogle(FAuthTokenGoogle&&) = default;
	FAuthTokenGoogle(const FAuthTokenGoogle&) = default;
	FAuthTokenGoogle& operator=(FAuthTokenGoogle&&) = default;
	FAuthTokenGoogle& operator=(const FAuthTokenGoogle&) = default;

	/**
	 * Parse a Google json auth response into an access/refresh token
	 *
	 * @param InJsonStr json response containing the token information
	 *
	 * @return true if parsing was successful, false otherwise
	 */
	bool Parse(const FString& InJsonStr);

	/**
	 * Parse a Google json auth response into an access/refresh token
	 *
	 * @param InJsonObject json object containing the token information
	 *
	 * @return true if parsing was successful, false otherwise
	 */
	bool Parse(TSharedPtr<FJsonObject> InJsonObject);

	/**
	 * Parse a Google json auth refresh response into an access/refresh token
	 *
	 * @param InJsonStr json response containing the token information
	 * @param InOldAuthToken previous auth token with refresh token information
	 *
	 * @return true if parsing was successful, false otherwise
	 */
	bool Parse(const FString& InJsonStr, const FAuthTokenGoogle& InOldAuthToken);

	/** @return true if this auth token is valid, false otherwise */
	bool IsValid() const
	{
		if (AuthType == EGoogleAuthTokenType::ExchangeToken)
		{
			return !AccessToken.IsEmpty() && RefreshToken.IsEmpty();
		}
		else if (AuthType == EGoogleAuthTokenType::RefreshToken)
		{
			return AccessToken.IsEmpty() && !RefreshToken.IsEmpty();
		}
		else
		{
			return !AccessToken.IsEmpty();
		}
	}

	/** @return true if the token is expired */
	bool IsExpired() const
	{
		if (AuthType == EGoogleAuthTokenType::AccessToken)
		{
			return (FDateTime::UtcNow() > ExpiresInUTC);
		}

		// For now don't worry about refresh/exchange token expiration
		return false;
	}

	inline bool GetAuthData(const FString& Key, FString& OutVal) const
	{
		const FString* FoundVal = AuthData.Find(Key);
		if (FoundVal != NULL)
		{
			OutVal = *FoundVal;
			return true;
		}
		return false;
	}

	inline void AddAuthData(FString Key, const FString& Value)
	{
		AuthData.Add(MoveTemp(Key), Value);
	}

	/** Type of auth this token represents */
	EGoogleAuthTokenType AuthType;
	/** Access or exchange token */
	FString AccessToken;
	/** Type of token (valid for AccessToken only) */
	FString TokenType;
	/** Number of seconds until this token expires at time of receipt */
	double ExpiresIn;
	/** Refresh token for generating new AccessTokens */
	FString RefreshToken;
	/** Id token in JWT form */
	FString IdToken;
	/** Parsed IdToken */
	FJsonWebTokenGoogle IdTokenJWT;
	/** Absolute time, in UTC, when this token will expire */
	FDateTime ExpiresInUTC;

private:

	void AddAuthAttributes(const TSharedPtr<FJsonObject>& JsonUser);

	/** Any addition auth data associated with the token */
	FJsonSerializableKeyValueMap AuthData;

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("access_token", AccessToken);
		JSON_SERIALIZE("token_type", TokenType);
		JSON_SERIALIZE("expires_in", ExpiresIn);
		JSON_SERIALIZE("refresh_token", RefreshToken);
		JSON_SERIALIZE("id_token", IdToken);
	END_JSON_SERIALIZER
};

/**
 * Container for configuration info related to all Google API services
 */
class FGoogleOpenIDConfiguration :
	public FJsonSerializable
{
public:

	/**
	 * Constructor
	 */
	FGoogleOpenIDConfiguration()
		: bInitialized(false)
		, AuthEndpoint(TEXT("https://accounts.google.com/o/oauth2/v2/auth"))
		, TokenEndpoint(TEXT("https://www.googleapis.com/oauth2/v4/token"))
		, UserInfoEndpoint(TEXT("https://www.googleapis.com/oauth2/v2/userinfo")) //"https://www.googleapis.com/userinfo/v2/me"
		, RevocationEndpoint(TEXT("https://accounts.google.com/o/oauth2/revoke"))
	{
	}

	/** @return true if this data is valid, false otherwise */
	bool IsValid() const { return bInitialized; }

	/**
	 * Parse a Json response from Google into a this data structure
	 *
	 * @return true if parsing was successful, false otherwise
	 */
	bool Parse(const FString& InJsonStr)
	{
		if (!InJsonStr.IsEmpty() && FromJson(InJsonStr))
		{
			bInitialized = true;
		}

		return bInitialized;
	}

	/** Has this data been setup */
	bool bInitialized;
	/** Issuer of the configuration information */
	FString Issuer;
	/** Authentication endpoint for login */
	FString AuthEndpoint;
	/** Token exchange endpoint */
	FString TokenEndpoint;
	/** User profile request endpoint */
	FString UserInfoEndpoint;
	/** Auth revocation endpoint */
	FString RevocationEndpoint;
	/** JWT Cert endpoint */
	FString JWKSURI;

private:

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("issuer", Issuer);
		JSON_SERIALIZE("authorization_endpoint", AuthEndpoint);
		JSON_SERIALIZE("token_endpoint", TokenEndpoint);
		JSON_SERIALIZE("userinfo_endpoint", UserInfoEndpoint);
		JSON_SERIALIZE("revocation_endpoint", RevocationEndpoint);
		JSON_SERIALIZE("jwks_uri", JWKSURI);
	END_JSON_SERIALIZER
};

/**
 * Google error from JSON payload
 */
class FErrorGoogle :
	public FJsonSerializable
{
public:

	/**
	 * Constructor
	 */
	FErrorGoogle()
	{
	}
	
	/** Error type */
	FString Error;
	/** Description of error */
	FString Error_Description;

	FString ToDebugString() const { return FString::Printf(TEXT("%s [Desc:%s]"), *Error, *Error_Description); }

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("error", Error);
		JSON_SERIALIZE("error_description", Error_Description);
	END_JSON_SERIALIZER
};
