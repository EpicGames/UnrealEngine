// Copyright Epic Games, Inc. All Rights Reserved.

#if USES_RESTFUL_GOOGLE

#include "OnlineExternalUIInterfaceGoogleRest.h"
#include "OnlineSubsystemGoogle.h"
#include "OnlineIdentityGoogleRest.h"
#include "OnlineError.h"

#define GOOGLE_STATE_TOKEN TEXT("state")
#define GOOGLE_ACCESS_TOKEN TEXT("code")
#define GOOGLE_ERRORCODE_TOKEN TEXT("error")
#define GOOGLE_ERRORCODE_DENY TEXT("access_denied")

bool FOnlineExternalUIGoogle::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	FString ErrorStr;
	bool bStarted = false;
	if (ControllerIndex >= 0 && ControllerIndex < MAX_LOCAL_PLAYERS)
	{
		FOnlineIdentityGooglePtr IdentityInt = StaticCastSharedPtr<FOnlineIdentityGoogle>(GoogleSubsystem->GetIdentityInterface());
		if (IdentityInt.IsValid())
		{
			IdentityInt->RetrieveDiscoveryDocument([this, ControllerIndex, Delegate](bool bWasSuccessful)
				{
					FString ErrorStr;
					if (bWasSuccessful)
					{
						FOnlineIdentityGooglePtr IdentityInt = StaticCastSharedPtr<FOnlineIdentityGoogle>(GoogleSubsystem->GetIdentityInterface());
						IdentityInt->UpdateLoginURLWithEndpointData();
						const FGoogleLoginURL& URLDetails = IdentityInt->GetLoginURLDetails();
						if (URLDetails.IsValid())
						{
							const FString RequestedURL = URLDetails.GetURL();
							bool bShouldContinueLoginFlow = false;
							FOnLoginRedirectURL OnRedirectURLDelegate = FOnLoginRedirectURL::CreateRaw(this, &FOnlineExternalUIGoogle::OnLoginRedirectURL);
							FOnLoginFlowComplete OnExternalLoginFlowCompleteDelegate = FOnLoginFlowComplete::CreateRaw(this, &FOnlineExternalUIGoogle::OnExternalLoginFlowComplete, ControllerIndex, Delegate);
							TriggerOnLoginFlowUIRequiredDelegates(RequestedURL, OnRedirectURLDelegate, OnExternalLoginFlowCompleteDelegate, bShouldContinueLoginFlow);
							if (bShouldContinueLoginFlow)
							{
								return;
							}
							else
							{
								ErrorStr = TEXT("ShowLoginUI: Login Flow interrupted");
							}
						}
						else
						{
							ErrorStr = TEXT("ShowLoginUI: Url Details not properly configured");
						}
					}
					else
					{
						ErrorStr = TEXT("ShowLoginUI: Could not retrieve Discovery Document");
					}

					FOnlineError Error;
					Error.SetFromErrorCode(TEXT(""));

					GoogleSubsystem->ExecuteNextTick([ControllerIndex, Delegate, Error = MoveTemp(Error)]()
						{
							Delegate.ExecuteIfBound(nullptr, ControllerIndex, Error);
						});
				});
				bStarted = true;
		}
		else
		{
			ErrorStr = TEXT("ShowLoginUI: Missing identity interface");
		}
	}
	else
	{
		ErrorStr = FString::Printf(TEXT("ShowLoginUI: Invalid controller index (%d)"), ControllerIndex);
	}

	if (!bStarted)
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("%s"), *ErrorStr);

		FOnlineError Error;
		Error.SetFromErrorCode(MoveTemp(ErrorStr));

		GoogleSubsystem->ExecuteNextTick([ControllerIndex, Delegate, Error = MoveTemp(Error)]()
		{
			Delegate.ExecuteIfBound(nullptr, ControllerIndex, Error);
		});
	}

	return bStarted;
}

FLoginFlowResult FOnlineExternalUIGoogle::OnLoginRedirectURL(const FString& RedirectURL)
{
	FLoginFlowResult Result;
	FOnlineIdentityGooglePtr IdentityInt = StaticCastSharedPtr<FOnlineIdentityGoogle>(GoogleSubsystem->GetIdentityInterface());
	if (IdentityInt.IsValid())
	{  
		const FGoogleLoginURL& URLDetails = IdentityInt->GetLoginURLDetails();
		if (URLDetails.IsValid())
		{
			// Wait for the RedirectURI to appear
			if (!RedirectURL.Contains(FPlatformHttp::UrlEncode(URLDetails.LoginUrl)) && RedirectURL.StartsWith(URLDetails.LoginRedirectUrl))
			{
				TMap<FString, FString> ParamsMap;

				{
					FString URLPrefix;
					FString ParamsOnly;
					if (!RedirectURL.Split(TEXT("?"), &URLPrefix, &ParamsOnly))
					{
						ParamsOnly = RedirectURL;
					}

					if (ParamsOnly[ParamsOnly.Len() - 1] == TEXT('#'))
					{
						ParamsOnly[ParamsOnly.Len() - 1] = TEXT('\0');
					}

					TArray<FString> Params;
					ParamsOnly.ParseIntoArray(Params, TEXT("&"));
					for (FString& Param : Params)
					{
						FString Key, Value;
						if (Param.Split(TEXT("="), &Key, &Value))
						{
							ParamsMap.Add(Key, Value);
						}
					}
				}

				const FString* State = ParamsMap.Find(GOOGLE_STATE_TOKEN);
				if ((URLDetails.State.IsEmpty() && !State) ||
					(!URLDetails.State.IsEmpty() && State && URLDetails.State == *State))
				{
					const FString* AccessToken = ParamsMap.Find(GOOGLE_ACCESS_TOKEN);
					if (AccessToken)
					{
						Result.Error.bSucceeded = true;
						Result.Token = *AccessToken;
					}
					else
					{
						const FString* ErrorCode = ParamsMap.Find(GOOGLE_ERRORCODE_TOKEN);
						if (ErrorCode)
						{
							if (*ErrorCode == GOOGLE_ERRORCODE_DENY)
							{
								Result.Error.ErrorRaw = LOGIN_CANCELLED;
								Result.Error.ErrorMessage = FText::FromString(LOGIN_CANCELLED);
								Result.Error.ErrorCode = LOGIN_CANCELLED;
								Result.Error.ErrorMessage = NSLOCTEXT("GoogleAuth", "GoogleAuthDeny", "Google Auth Denied");
								Result.NumericErrorCode = -1;
							}
							else
							{
								Result.Error.ErrorRaw = RedirectURL;
								Result.Error.ErrorCode = *ErrorCode;
								// there is no descriptive error text
								Result.Error.ErrorMessage = NSLOCTEXT("GoogleAuth", "GoogleAuthError", "Google Auth Error");
								// there is no error code
								Result.NumericErrorCode = 0;
							}
						}
						else
						{
							// Set some default in case parsing fails
							Result.Error.ErrorRaw = LOGIN_ERROR_UNKNOWN;
							Result.Error.ErrorMessage = FText::FromString(LOGIN_ERROR_UNKNOWN);
							Result.Error.ErrorCode = LOGIN_ERROR_UNKNOWN;
							Result.NumericErrorCode = -2;
						}
					}
				}
				else
				{
					// Set some default in case parsing fails
					Result.Error.ErrorRaw = LOGIN_ERROR_UNKNOWN;
					Result.Error.ErrorMessage = FText::FromString(LOGIN_ERROR_UNKNOWN);
					Result.Error.ErrorCode = LOGIN_ERROR_UNKNOWN;
					Result.NumericErrorCode = -2;
				}
			}
		}
	}

	return Result;
}

void FOnlineExternalUIGoogle::OnExternalLoginFlowComplete(const FLoginFlowResult& Result, int ControllerIndex, const FOnLoginUIClosedDelegate Delegate)
{
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("OnExternalLoginFlowComplete %s"), *Result.ToDebugString());

	bool bStarted = false;
	if (Result.IsValid())
	{
		FOnlineIdentityGooglePtr IdentityInt = StaticCastSharedPtr<FOnlineIdentityGoogle>(GoogleSubsystem->GetIdentityInterface());
		if (IdentityInt.IsValid())
		{
			bStarted = true;

			FOnLoginCompleteDelegate CompletionDelegate;
			CompletionDelegate = FOnLoginCompleteDelegate::CreateRaw(this, &FOnlineExternalUIGoogle::OnAccessTokenLoginComplete, Delegate);

			FAuthTokenGoogle AuthToken(Result.Token, EGoogleExchangeToken::GoogleExchangeToken);
			IdentityInt->Login(ControllerIndex, AuthToken, CompletionDelegate);
		}
	}

	if (!bStarted)
	{
		FOnlineError LoginFlowError = Result.Error;
		GoogleSubsystem->ExecuteNextTick([ControllerIndex, LoginFlowError, Delegate]()
		{
			Delegate.ExecuteIfBound(nullptr, ControllerIndex, LoginFlowError);
		});
	}
}

void FOnlineExternalUIGoogle::OnAccessTokenLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error, FOnLoginUIClosedDelegate Delegate)
{
	FUniqueNetIdPtr StrongUserId = UserId.AsShared();
	GoogleSubsystem->ExecuteNextTick([StrongUserId, LocalUserNum, bWasSuccessful, Delegate]()
	{
		Delegate.ExecuteIfBound(StrongUserId, LocalUserNum, FOnlineError(bWasSuccessful));
	});
}

#endif // USES_RESTFUL_GOOGLE
