// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USES_RESTFUL_GOOGLE

#include "OnlineSubsystemGoogle.h"
#include "OnlineExternalUIGoogleCommon.h"
#include "OnlineSubsystemGooglePackage.h"

class FOnlineSubsystemGoogle;

/** 
 * Implementation for the Google external UIs
 */
class FOnlineExternalUIGoogle : public FOnlineExternalUIGoogleCommon
{

PACKAGE_SCOPE:

	/** 
	 * Constructor
	 * @param InSubsystem The owner of this external UI interface.
	 */
	FOnlineExternalUIGoogle(FOnlineSubsystemGoogle* InSubsystem) :
		FOnlineExternalUIGoogleCommon(InSubsystem)
	{
	}

public:

	/**
	 * Destructor.
	 */
	virtual ~FOnlineExternalUIGoogle()
	{
	}

	// IOnlineExternalUI
	virtual bool ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate = FOnLoginUIClosedDelegate()) override;

private: 

	/**
	 * Delegate fired when redirect URLs from the login flow are passed back for parsing
	 * We are looking for the success or error completion state from Google to grab the access token or complete the flow
	 *
	 * @param RedirectURL URL received from the login flow for parsing
	 */
	FLoginFlowResult OnLoginRedirectURL(const FString& RedirectURL);

	/**
	 * Delegate fired when the login flow is complete
	 *
	 * @param Result final result of the login flow action
	 * @param ControllerIndex index of the local user initiating the request
	 * @param Delegate UI closed delegate to fire, completing the external UIs part in the login process
	 */
	void OnExternalLoginFlowComplete(const FLoginFlowResult& Result, int ControllerIndex, const FOnLoginUIClosedDelegate Delegate);

	/**
	 * Delegate fired when the Google identity interface has completed login using the token retrieved from the login flow
	 *
	 * @param LocalUserNum index of the local user initiating the request
	 * @param bWasSuccessful was the login call successful
	 * @param UserId user id of the logged in user, or null if login failed
	 * @param Error error string if applicable
	 * @param Delegate UI closed delegate to fire, completing the external UIs part in the login process
	 */
	void OnAccessTokenLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error, FOnLoginUIClosedDelegate Delegate);
};

typedef TSharedPtr<FOnlineExternalUIGoogle, ESPMode::ThreadSafe> FOnlineExternalUIGooglePtr;

#endif // USES_RESTFUL_GOOGLE
