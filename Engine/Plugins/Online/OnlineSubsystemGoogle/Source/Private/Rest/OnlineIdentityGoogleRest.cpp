// Copyright Epic Games, Inc. All Rights Reserved.

#if USES_RESTFUL_GOOGLE

#include "OnlineIdentityGoogleRest.h"
#include "OnlineSubsystemGooglePrivate.h"
#include "OnlineExternalUIInterfaceGoogleRest.h"
#include "Misc/ConfigCacheIni.h"

FOnlineIdentityGoogle::FOnlineIdentityGoogle(FOnlineSubsystemGoogle* InSubsystem)
	: FOnlineIdentityGoogleCommon(InSubsystem)
	, bHasLoginOutstanding(false)
{
	if (!GConfig->GetString(TEXT("OnlineSubsystemGoogle"), TEXT("ClientSecret"), ClientSecret, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing ClientSecret= in [OnlineSubsystemGoogle] of DefaultEngine.ini"));
	}
	if (!GConfig->GetString(TEXT("OnlineSubsystemGoogle.OnlineIdentityGoogle"), TEXT("LoginRedirectUrl"), LoginURLDetails.LoginRedirectUrl, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing LoginRedirectUrl= in [OnlineSubsystemGoogle.OnlineIdentityGoogle] of DefaultEngine.ini"));
	}
	if (!GConfig->GetInt(TEXT("OnlineSubsystemGoogle.OnlineIdentityGoogle"), TEXT("RedirectPort"), LoginURLDetails.RedirectPort, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing RedirectPort= in [OnlineSubsystemGoogle.OnlineIdentityGoogle] of DefaultEngine.ini"));
	}

	GConfig->GetArray(TEXT("OnlineSubsystemGoogle.OnlineIdentityGoogle"), TEXT("LoginDomains"), LoginDomains, GEngineIni);

	LoginURLDetails.ClientId = InSubsystem->GetAppId();

	// Setup permission scope fields
	GConfig->GetArray(TEXT("OnlineSubsystemGoogle.OnlineIdentityGoogle"), TEXT("ScopeFields"), LoginURLDetails.ScopeFields, GEngineIni);
	// always required login access fields
	LoginURLDetails.ScopeFields.AddUnique(TEXT(GOOGLE_PERM_PUBLIC_PROFILE));
	LoginURLDetails.bRequestOfflineAccess = ShouldRequestOfflineAccess();
}

void FOnlineIdentityGoogle::DiscoveryRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, PendingLoginRequestCb LoginCb)
{
	FOnlineIdentityGoogleCommon::DiscoveryRequest_HttpRequestComplete(HttpRequest, HttpResponse, bSucceeded, LoginCb);
	if (Endpoints.IsValid())
	{
		LoginURLDetails.LoginUrl = Endpoints.AuthEndpoint;
	}
}

bool FOnlineIdentityGoogle::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	FString ErrorStr;

	if (bHasLoginOutstanding)
	{
		ErrorStr = FString::Printf(TEXT("Registration already pending for user"));
	}
	else if (!LoginURLDetails.IsValid())
	{
		ErrorStr = FString::Printf(TEXT("OnlineSubsystemGoogle is improperly configured in DefaultEngine.ini LoginRedirectUrl=%s RedirectPort=%d ClientId=%s"),
			*LoginURLDetails.LoginRedirectUrl, LoginURLDetails.RedirectPort, *LoginURLDetails.ClientId);
	}
	else
	{
		if (LocalUserNum < 0 || LocalUserNum >= MAX_LOCAL_PLAYERS)
		{
			ErrorStr = FString::Printf(TEXT("Invalid LocalUserNum=%d"), LocalUserNum);
		}
		else
		{
			PendingLoginRequestCb PendingLoginFn;
			if (!AccountCredentials.Id.IsEmpty() && !AccountCredentials.Token.IsEmpty() && AccountCredentials.Type == GetAuthType())
			{
				const FString AccessToken = AccountCredentials.Token;
				PendingLoginFn = [this, LocalUserNum, AccessToken](bool bWasSuccessful)
				{
					if (bWasSuccessful)
					{
						LoginURLDetails.LoginUrl = Endpoints.AuthEndpoint;

						FAuthTokenGoogle AuthToken(AccessToken, EGoogleRefreshToken::GoogleRefreshToken);
						Login(LocalUserNum, AuthToken, FOnLoginCompleteDelegate::CreateRaw(this, &FOnlineIdentityGoogle::OnAccessTokenLoginComplete));
					}
					else
					{
						const FString ErrorStr = TEXT("Error retrieving discovery service");
						OnLoginAttemptComplete(LocalUserNum, ErrorStr);
					}
				};
			}
			else
			{
				PendingLoginFn = [this, LocalUserNum](bool bWasSuccessful)
				{
					if (bWasSuccessful)
					{
						LoginURLDetails.LoginUrl = Endpoints.AuthEndpoint;
						IOnlineExternalUIPtr OnlineExternalUI = GoogleSubsystem->GetExternalUIInterface();
						if (ensure(OnlineExternalUI.IsValid()))
						{
							LoginURLDetails.GenerateNonce();
							FOnLoginUIClosedDelegate CompletionDelegate = FOnLoginUIClosedDelegate::CreateRaw(this, &FOnlineIdentityGoogle::OnExternalUILoginComplete);
							OnlineExternalUI->ShowLoginUI(LocalUserNum, true, false, CompletionDelegate);
						}
					}
					else
					{
						const FString ErrorStr = TEXT("Error retrieving discovery service");
						OnLoginAttemptComplete(LocalUserNum, ErrorStr);
					}
				};
			}

			bHasLoginOutstanding = true;
			RetrieveDiscoveryDocument(MoveTemp(PendingLoginFn));
		}
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Error, TEXT("FOnlineIdentityGoogle::Login() failed: %s"),	*ErrorStr);
		OnLoginAttemptComplete(LocalUserNum, ErrorStr);
		return false;
	}
	return true;
}

void FOnlineIdentityGoogle::Login(int32 LocalUserNum, const FAuthTokenGoogle& InToken, const FOnLoginCompleteDelegate& InCompletionDelegate)
{	
	TFunction<void(int32, bool, const FAuthTokenGoogle&, const FString&)> NextFn = 
	[this, InCompletionDelegate](int32 InLocalUserNum, bool bWasSuccessful, const FAuthTokenGoogle& InAuthToken, const FString& ErrorStr)
	{
		FOnProfileRequestComplete ProfileCompletionDelegate = FOnProfileRequestComplete::CreateLambda([this, InCompletionDelegate](int32 ProfileLocalUserNum, bool bWasSuccessful, const FString& ErrorStr)
		{
			if (bWasSuccessful)
			{
				InCompletionDelegate.ExecuteIfBound(ProfileLocalUserNum, bWasSuccessful, *GetUniquePlayerId(ProfileLocalUserNum), ErrorStr);
			}
			else
			{
				InCompletionDelegate.ExecuteIfBound(ProfileLocalUserNum, bWasSuccessful, *FUniqueNetIdGoogle::EmptyId(), ErrorStr);
			}
		});

		if (bWasSuccessful)
		{
			ProfileRequest(InLocalUserNum, InAuthToken, ProfileCompletionDelegate);
		}
		else
		{
			InCompletionDelegate.ExecuteIfBound(InLocalUserNum, bWasSuccessful, *FUniqueNetIdGoogle::EmptyId(), ErrorStr);
		}
	};

	if (InToken.AuthType == EGoogleAuthTokenType::ExchangeToken)
	{
		FOnExchangeRequestComplete CompletionDelegate = FOnExchangeRequestComplete::CreateLambda([NextFn](int32 InLocalUserNum, bool bWasSuccessful, const FAuthTokenGoogle& InAuthToken, const FString& ErrorStr)
		{
			NextFn(InLocalUserNum, bWasSuccessful, InAuthToken, ErrorStr);
		});

		ExchangeCode(LocalUserNum, InToken, CompletionDelegate);
	}
	else if (InToken.AuthType == EGoogleAuthTokenType::RefreshToken)
	{
		FOnRefreshAuthRequestComplete CompletionDelegate = FOnRefreshAuthRequestComplete::CreateLambda([NextFn](int32 InLocalUserNum, bool bWasSuccessful, const FAuthTokenGoogle& InAuthToken, const FString& ErrorStr)
		{
			NextFn(InLocalUserNum, bWasSuccessful, InAuthToken, ErrorStr);
		});

		RefreshAuth(LocalUserNum, InToken, CompletionDelegate);
	}
}

void FOnlineIdentityGoogle::OnAccessTokenLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UniqueId, const FString& Error)
{
	OnLoginAttemptComplete(LocalUserNum, Error);
}

void FOnlineIdentityGoogle::OnExternalUILoginComplete(FUniqueNetIdPtr UniqueId, const int ControllerIndex, const FOnlineError& Error)
{
	const FString& ErrorStr = Error.GetErrorCode();
	const bool bWasSuccessful = Error.WasSuccessful() && UniqueId.IsValid() && UniqueId->IsValid();
	OnAccessTokenLoginComplete(ControllerIndex, bWasSuccessful, bWasSuccessful ? *UniqueId : *FUniqueNetIdGoogle::EmptyId(), ErrorStr);
}

void FOnlineIdentityGoogle::OnLoginAttemptComplete(int32 LocalUserNum, const FString& ErrorStr)
{
	const FString ErrorStrCopy(ErrorStr);

	bHasLoginOutstanding = false;
	if (GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Google login was successful"));
		FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
		check(UserId.IsValid());

		GoogleSubsystem->ExecuteNextTick([this, UserId, LocalUserNum, ErrorStrCopy]()
		{
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, *UserId, ErrorStrCopy);
			TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, *UserId);
		});
	}
	else
	{
		GoogleSubsystem->ExecuteNextTick([this, LocalUserNum, ErrorStrCopy]()
		{
			TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdGoogle::EmptyId(), ErrorStrCopy);
		});
	}
}

void FOnlineIdentityGoogle::ExchangeCode(int32 LocalUserNum, const FAuthTokenGoogle& InExchangeToken, FOnExchangeRequestComplete& InCompletionDelegate)
{
	FString ErrorStr;
	bool bStarted = false;
	if (LocalUserNum >= 0 && LocalUserNum < MAX_LOCAL_PLAYERS)
	{
		if (Endpoints.IsValid() && !Endpoints.TokenEndpoint.IsEmpty())
		{
			if (InExchangeToken.IsValid())
			{
				check(InExchangeToken.AuthType == EGoogleAuthTokenType::ExchangeToken);
				bStarted = true;

				const FString RedirectURL = LoginURLDetails.GetRedirectURL();
				const FString PostContent = FString::Printf(TEXT("code=%s&client_id=%s&client_secret=%s&redirect_uri=%s&grant_type=authorization_code"),
					*InExchangeToken.AccessToken,
					*GoogleSubsystem->GetAppId(),
					*ClientSecret,
					*RedirectURL);

				// kick off http request to convert the exchange code to access token
				TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

				HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOnlineIdentityGoogle::ExchangeRequest_HttpRequestComplete, LocalUserNum, InExchangeToken, InCompletionDelegate);
				HttpRequest->SetURL(Endpoints.TokenEndpoint);
				HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/x-www-form-urlencoded"));
				HttpRequest->SetVerb(TEXT("POST"));

				HttpRequest->SetContentAsString(PostContent);
				HttpRequest->ProcessRequest();
			}
			else
			{
				ErrorStr = TEXT("No access token specified");
			}
		}
		else
		{
			ErrorStr = TEXT("Invalid Google endpoint");
		}
	}
	else
	{
		ErrorStr = TEXT("Invalid local user num");
	}

	if (!bStarted)
	{
		FAuthTokenGoogle EmptyAuthToken;
		InCompletionDelegate.ExecuteIfBound(LocalUserNum, false, EmptyAuthToken, ErrorStr);
	}
}

void FOnlineIdentityGoogle::ExchangeRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, int32 InLocalUserNum, FAuthTokenGoogle InExchangeToken, FOnExchangeRequestComplete InCompletionDelegate)
{
	bool bResult = false;
	FString ResponseStr, ErrorStr;

	FAuthTokenGoogle AuthToken;

	if (bSucceeded &&
		HttpResponse.IsValid())
	{
		ResponseStr = HttpResponse->GetContentAsString();
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
			UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("ExchangeCode request complete. url=%s code=%d response=%s"),
				*HttpRequest->GetURL(), HttpResponse->GetResponseCode(), *ResponseStr);

			if (AuthToken.Parse(ResponseStr))
			{
				bResult = true;
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("Failed to parse auth json"));
			}
		}
		else
		{
			FErrorGoogle Error;
			if (Error.FromJson(ResponseStr) && !Error.Error_Description.IsEmpty())
			{
				ErrorStr = Error.Error_Description;
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("Failed to parse Google error %s"), *ResponseStr);
			}
		}
	}
	else
	{
		ErrorStr = TEXT("No response");
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Exchange request failed. %s"), *ErrorStr);
	}

	InCompletionDelegate.ExecuteIfBound(InLocalUserNum, bResult, AuthToken, ErrorStr);
}

void FOnlineIdentityGoogle::RefreshAuth(int32 LocalUserNum, const FAuthTokenGoogle& InAuthToken, FOnRefreshAuthRequestComplete& InCompletionDelegate)
{
	FString ErrorStr;
	bool bStarted = false;
	if (LocalUserNum >= 0 && LocalUserNum < MAX_LOCAL_PLAYERS)
	{
		if (Endpoints.IsValid() && !Endpoints.TokenEndpoint.IsEmpty())
		{
			if (InAuthToken.IsValid())
			{
				check(InAuthToken.AuthType == EGoogleAuthTokenType::AccessToken || InAuthToken.AuthType == EGoogleAuthTokenType::RefreshToken);
				bStarted = true;

				const FString PostContent = FString::Printf(TEXT("client_id=%s&client_secret=%s&refresh_token=%s&grant_type=refresh_token"),
					*GoogleSubsystem->GetAppId(),
					*ClientSecret,
					*InAuthToken.RefreshToken);

				// kick off http request to refresh the auth token
				TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

				HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOnlineIdentityGoogle::RefreshAuthRequest_HttpRequestComplete, LocalUserNum, InAuthToken, InCompletionDelegate);
				HttpRequest->SetURL(Endpoints.TokenEndpoint);
				HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/x-www-form-urlencoded"));
				HttpRequest->SetVerb(TEXT("POST"));

				HttpRequest->SetContentAsString(PostContent);
				HttpRequest->ProcessRequest();
			}
			else
			{
				ErrorStr = TEXT("No access token specified");
			}
		}
		else
		{
			ErrorStr = TEXT("Invalid Google endpoint");
		}
	}
	else
	{
		ErrorStr = TEXT("Invalid local user num");
	}

	if (!bStarted)
	{
		FAuthTokenGoogle EmptyAuthToken;
		InCompletionDelegate.ExecuteIfBound(LocalUserNum, false, EmptyAuthToken, ErrorStr);
	}
}

void FOnlineIdentityGoogle::RefreshAuthRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, int32 InLocalUserNum, FAuthTokenGoogle InOldAuthToken, FOnRefreshAuthRequestComplete InCompletionDelegate)
{
	bool bResult = false;
	FString ResponseStr, ErrorStr;

	FAuthTokenGoogle AuthToken;

	if (bSucceeded &&
		HttpResponse.IsValid())
	{
		ResponseStr = HttpResponse->GetContentAsString();
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
			UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("RefreshAuth request complete. url=%s code=%d response=%s"),
				*HttpRequest->GetURL(), HttpResponse->GetResponseCode(), *ResponseStr);

			if (AuthToken.Parse(ResponseStr, InOldAuthToken))
			{
				bResult = true;
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("Failed to parse refresh auth json"));
			}
		}
		else
		{
			FErrorGoogle Error;
			if (Error.FromJson(ResponseStr) && !Error.Error_Description.IsEmpty())
			{
				ErrorStr = Error.Error_Description;
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("Failed to parse Google error %s"), *ResponseStr);
			}
		}
	}
	else
	{
		ErrorStr = TEXT("No response");
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("RefreshAuth request failed. %s"), *ErrorStr);
	}

	InCompletionDelegate.ExecuteIfBound(InLocalUserNum, bResult, AuthToken, ErrorStr);
}

bool FOnlineIdentityGoogle::Logout(int32 LocalUserNum)
{
	bool bResult = false;
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		// remove cached user account
		UserAccounts.Remove(UserId->ToString());
		// remove cached user id
		UserIds.Remove(LocalUserNum);
		// reset scope permissions
		GConfig->GetArray(TEXT("OnlineSubsystemGoogle.OnlineIdentityGoogle"), TEXT("ScopeFields"), LoginURLDetails.ScopeFields, GEngineIni);
		// always required login access fields
		LoginURLDetails.ScopeFields.AddUnique(TEXT(GOOGLE_PERM_PUBLIC_PROFILE));

		TriggerOnLoginFlowLogoutDelegates(LoginDomains);

		// not async but should call completion delegate anyway
		GoogleSubsystem->ExecuteNextTick([this, LocalUserNum, UserId]() 
		{
			TriggerOnLogoutCompleteDelegates(LocalUserNum, true);
			TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::LoggedIn, ELoginStatus::NotLoggedIn, *UserId);
		});

		bResult = true;
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("No logged in user found for LocalUserNum=%d."), LocalUserNum);
		GoogleSubsystem->ExecuteNextTick([this, LocalUserNum]() 
		{
			TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
		});
	}
	
	return bResult;
}

#endif // USES_RESTFUL_GOOGLE
