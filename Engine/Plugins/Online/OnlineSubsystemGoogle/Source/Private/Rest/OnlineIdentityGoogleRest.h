// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USES_RESTFUL_GOOGLE

#include "OnlineIdentityGoogleCommon.h"
#include "OnlineSubsystemGoogleTypes.h"
#include "OnlineAccountGoogleCommon.h"
#include "PlatformHttp.h"
#include "OnlineSubsystemGooglePackage.h"

class FOnlineSubsystemGoogle;

/**
 * Delegate fired after an exchange token to access/refresh token request has been completed
 *
 * @param LocalUserNum the controller number of the associated user
 * @param bWasSuccessful was the request successful
 * @param AuthToken the new auth token returned from the exchange
 * @param ErrorStr error associated with the request
 */
DECLARE_DELEGATE_FourParams(FOnExchangeRequestComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FAuthTokenGoogle& /*AuthToken*/, const FString& /*ErrorStr*/);

/**
 * Delegate fired after a refresh auth from existing token request has been completed
 *
 * @param LocalUserNum the controller number of the associated user
 * @param bWasSuccessful was the request successful
 * @param AuthToken the new auth token returned from the refresh 
 * @param ErrorStr error associated with the request
 */
DECLARE_DELEGATE_FourParams(FOnRefreshAuthRequestComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FAuthTokenGoogle& /*AuthToken*/, const FString& /*ErrorStr*/);

/** Windows implementation of a Google user account */
class FUserOnlineAccountGoogle : public FUserOnlineAccountGoogleCommon
{
public:

	explicit FUserOnlineAccountGoogle(const FString& InUserId = FString(), const FAuthTokenGoogle& InAuthToken = FAuthTokenGoogle())
		: FUserOnlineAccountGoogleCommon(InUserId, InAuthToken)
	{
	}

	virtual ~FUserOnlineAccountGoogle()
	{
	}
};

/**
 * Contains URL details for Windows Google interaction
 */
struct FGoogleLoginURL
{
	/** The Google exchange token auth endpoint */
	FString LoginUrl;
	/** The redirect url for Google to redirect to upon completion */
	FString LoginRedirectUrl;
	/** Port to append to the LoginRedirectURL when communicating with Google auth services */
	int32 RedirectPort = 9000;
	/** The client id given to us by Google */
	FString ClientId;
	/** Config based list of permission scopes to use when logging in */
	TArray<FString> ScopeFields;
	/** A value used to verify our response came from our server */
	FString State;
	/** Should request Offline Access. Refresh token is only received on first authorization */
	bool bRequestOfflineAccess = false;

	bool IsValid() const
	{
		// LoginUrl skipped because its filled in by discovery service
		return !LoginRedirectUrl.IsEmpty() && !ClientId.IsEmpty() && (RedirectPort > 0) && (ScopeFields.Num() > 0);
	}

	FString GenerateNonce()
	{
		// random number to represent client generated state for verification on login
		State = FString::FromInt(FMath::Rand() % 100000);
		return State;
	}

	FString GetRedirectURL() const
	{
		//const FString Redirect = TEXT("urn:ietf:wg:oauth:2.0:oob:auto");
		return FString::Printf(TEXT("%s:%d"), *LoginRedirectUrl, RedirectPort);
	}

	FString GetURL() const
	{
		FString Scopes = FString::Join(ScopeFields, TEXT(" "));
		
		const FString Redirect = GetRedirectURL();
		
		const FString ParamsString = FString::Printf(TEXT("redirect_uri=%s&scope=%s&response_type=code&client_id=%s&state=%s"),
			*Redirect, *FPlatformHttp::UrlEncode(Scopes), *ClientId, *State);

		// auth url to spawn in browser
		const FString URLString = FString::Printf(TEXT("%s?%s%s"),
			*LoginUrl, *ParamsString, bRequestOfflineAccess?TEXT("&access_type=offline"):TEXT(""));
		
		return URLString;
	}
};

/**
 * Google service implementation of the online identity interface
 */
class FOnlineIdentityGoogle :
	public FOnlineIdentityGoogleCommon
{
	/** Client secret retrieved from Google Dashboard */
	FString ClientSecret;
	/** Const details about communicating with Google API */
	FGoogleLoginURL LoginURLDetails;
	/** Whether we have a registration in flight or not */
	bool bHasLoginOutstanding;

	/** Domains used for login, for cookie management */
	TArray<FString> LoginDomains;

public:
	// IOnlineIdentity
	
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;

	// FOnlineIdentityGoogle

	FOnlineIdentityGoogle(FOnlineSubsystemGoogle* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityGoogle()
	{
	}

PACKAGE_SCOPE:

	void UpdateLoginURLWithEndpointData() { LoginURLDetails.LoginUrl = Endpoints.AuthEndpoint; };

	/**
	 * Login with an existing token
	 * If an exchange token has been given, it will convert to access/refresh before continuing login
	 * If an older access token has been given, it will refresh the token before continuing login
	 *
	 * @param LocalUserNum id of the local user initiating the request
	 * @param InToken exchange or refresh token already received from Google
	 * @param InCompletionDelegate delegate to fire when operation completes
	 */
	void Login(int32 LocalUserNum, const FAuthTokenGoogle& InToken, const FOnLoginCompleteDelegate& InCompletionDelegate);

	/** @return the login configuration details */
	const FGoogleLoginURL& GetLoginURLDetails() const { return LoginURLDetails; }

	/**
	 * Exchange the Google auth token for actual user access/refresh tokens
	 *
	 * @param LocalUserNum id of the local user initiating the request
	 * @param InExchangeToken token received from user consent login flow
	 * @param InCompletionDelegate delegate to fire when operation completes
	 */
	void ExchangeCode(int32 LocalUserNum, const FAuthTokenGoogle& InExchangeToken, FOnExchangeRequestComplete& InCompletionDelegate);

	/**
	 * Refresh an existing Google auth token
	 *
	 * @param LocalUserNum id of the local user initiating the request
	 * @param InAuthToken existing valid auth token with refresh token included
	 * @param InCompletionDelegate delegate to fire when operation completes
	 */
	void RefreshAuth(int32 LocalUserNum, const FAuthTokenGoogle& InAuthToken, FOnRefreshAuthRequestComplete& InCompletionDelegate);
	
private:

	// FOnlineIdentityGoogleCommon
	virtual void DiscoveryRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, PendingLoginRequestCb LoginCb) override;


	void OnLoginAttemptComplete(int32 LocalUserNum, const FString& ErrorStr);

	/**
	 * Delegate called when a token exchange has completed
	 */
	void ExchangeRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, int32 InLocalUserNum, FAuthTokenGoogle InExchangeToken, FOnExchangeRequestComplete InCompletionDelegate);

	/**
	 * Delegate called when a refresh auth request has completed
	 */
	void RefreshAuthRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, int32 InLocalUserNum, FAuthTokenGoogle InOldAuthToken, FOnRefreshAuthRequestComplete InCompletionDelegate);

	/**
	 * Delegate fired when the internal call to Login() with AccessToken is specified
	 *
	 * @param LocalUserNum index of the local user initiating the request
	 * @param bWasSuccessful was the login call successful
	 * @param UserId user id of the logged in user, or null if login failed
	 * @param Error error string if applicable
	 */
	void OnAccessTokenLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error);

	/**
	 * Delegate fired when the call to ShowLoginUI completes 
	 */
	void OnExternalUILoginComplete(FUniqueNetIdPtr UniqueId, const int ControllerIndex, const FOnlineError& Error);

};

typedef TSharedPtr<FOnlineIdentityGoogle, ESPMode::ThreadSafe> FOnlineIdentityGooglePtr;

#endif // USES_RESTFUL_GOOGLE
