// Copyright Epic Games, Inc. All Rights Reserved.

#if USES_RESTFUL_GOOGLE

#include "OnlineSubsystemGoogle.h"
#include "OnlineSubsystemGooglePrivate.h"
#include "OnlineIdentityGoogleRest.h"
#include "OnlineExternalUIInterfaceGoogleRest.h"

FOnlineSubsystemGoogle::FOnlineSubsystemGoogle(FName InInstanceName)
	: FOnlineSubsystemGoogleCommon(InInstanceName)
{
	bPlatformRequiresClientId = true;
	bPlatformAllowsClientIdOverride = true;
	bPlatformRequiresServerClientId = false;
}

FOnlineSubsystemGoogle::~FOnlineSubsystemGoogle()
{
}

bool FOnlineSubsystemGoogle::Init()
{
	if (FOnlineSubsystemGoogleCommon::Init())
	{
		GoogleIdentity = MakeShareable(new FOnlineIdentityGoogle(this));
		GoogleExternalUI = MakeShareable(new FOnlineExternalUIGoogle(this));
		return true;
	}

	return false;
}

bool FOnlineSubsystemGoogle::Shutdown()
{
	UE_LOG_ONLINE(Display, TEXT("FOnlineSubsystemGoogle::Shutdown()"));
	return FOnlineSubsystemGoogleCommon::Shutdown();
}

#endif // USES_RESTFUL_GOOGLE
