// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemGoogleCommon.h"
#include "OnlineSubsystemGooglePackage.h"

/** Forward declarations of all interface classes */
typedef TSharedPtr<class FOnlineIdentityGoogle, ESPMode::ThreadSafe> FOnlineIdentityGooglePtr;
typedef TSharedPtr<class FOnlineFriendsGoogle, ESPMode::ThreadSafe> FOnlineFriendsGooglePtr;
typedef TSharedPtr<class FOnlineSharingGoogle, ESPMode::ThreadSafe> FOnlineSharingGooglePtr;
typedef TSharedPtr<class FOnlineUserGoogle, ESPMode::ThreadSafe> FOnlineUserGooglePtr;
typedef TSharedPtr<class FOnlineExternalUIGoogle, ESPMode::ThreadSafe> FOnlineExternalUIGooglePtr;

/**
 *	OnlineSubsystemGoogle - Implementation of the online subsystem for Google services
 */
class ONLINESUBSYSTEMGOOGLE_API FOnlineSubsystemGoogle 
	: public FOnlineSubsystemGoogleCommon
{
public:

	// FOnlineSubsystemGoogleCommon Interface
	virtual bool Init() override;
	virtual bool Shutdown() override;

	// FOnlineSubsystemGoogle

	/**
	 * Destructor
	 */
	virtual ~FOnlineSubsystemGoogle();

PACKAGE_SCOPE:

	/** Only the factory makes instances */
	FOnlineSubsystemGoogle() = delete;
	explicit FOnlineSubsystemGoogle(FName InInstanceName);

};

typedef TSharedPtr<FOnlineSubsystemGoogle, ESPMode::ThreadSafe> FOnlineSubsystemGooglePtr;
