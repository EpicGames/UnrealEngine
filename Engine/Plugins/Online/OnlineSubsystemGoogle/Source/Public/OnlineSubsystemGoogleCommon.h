// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"
#include "OnlineSubsystemImpl.h"
#include "Serialization/JsonSerializerMacros.h"
#include "OnlineSubsystemGooglePackage.h"

/** Forward declarations of all interface classes */
typedef TSharedPtr<class FOnlineIdentityGoogleCommon, ESPMode::ThreadSafe> FOnlineIdentityGoogleCommonPtr;
typedef TSharedPtr<class FOnlineExternalUIGoogleCommon, ESPMode::ThreadSafe> FOnlineExternalUIGoogleCommonPtr;

/**
 * Configuration structure for overriding auth id parameters required by Google
 */
class FGoogleAuthConfig
{
public:
	FGoogleAuthConfig() {}

	/** 
	 * Backend Name
	 *
	 * Added to [OnlineSubsystemGoogle <Backend>] to search the ini for specific overrides
	 */
	FString Backend;
};

/**
 *	OnlineSubsystemGoogleCommon - Implementation of the online subsystem for Google services
 */
class ONLINESUBSYSTEMGOOGLE_API FOnlineSubsystemGoogleCommon 
	: public FOnlineSubsystemImpl
{
public:

	// IOnlineSubsystem Interface
	virtual IOnlineSessionPtr GetSessionInterface() const override;
	virtual IOnlineFriendsPtr GetFriendsInterface() const override;
	virtual IOnlinePartyPtr GetPartyInterface() const override;
	virtual IOnlineGroupsPtr GetGroupsInterface() const override;
	virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	virtual IOnlineVoicePtr GetVoiceInterface() const override;
	virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;
	virtual IOnlineTimePtr GetTimeInterface() const override;
	virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override { return nullptr; }
	virtual IOnlinePurchasePtr GetPurchaseInterface() const override { return nullptr; }
	virtual IOnlineEventsPtr GetEventsInterface() const override;
	virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	virtual IOnlineSharingPtr GetSharingInterface() const override;
	virtual IOnlineUserPtr GetUserInterface() const override;
	virtual IOnlineMessagePtr GetMessageInterface() const override;
	virtual IOnlinePresencePtr GetPresenceInterface() const override;
	virtual IOnlineChatPtr GetChatInterface() const override;
	virtual IOnlineStatsPtr GetStatsInterface() const override;
	virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override;
	virtual IOnlineTournamentPtr GetTournamentInterface() const override;
	virtual FText GetOnlineServiceName() const override;
	virtual bool Init() override;
	virtual bool Shutdown() override;
	virtual FString GetAppId() const override;
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

	// FTickerBaseObject

	virtual bool Tick(float DeltaTime) override;

	// FOnlineSubsystemGoogleCommon

	/**
	 * Destructor
	 */
	virtual ~FOnlineSubsystemGoogleCommon();

	/**
	 * Delegate fired at initialization allowing the application to override the configuration parameters 
	 *
	 * @param ConfigOverride unused
	 * @param OutConfig config parameters needed to be filled in in order to override the default engine ini settings
	 */
	DECLARE_DELEGATE_TwoParams(FGoogleConfigurationDelegate, const FString& /*ConfigOverride*/, FGoogleAuthConfig& /*OutConfig*/);
	static FOnlineSubsystemGoogleCommon::FGoogleConfigurationDelegate& GetConfigurationDelegate();

PACKAGE_SCOPE:

	/** Only the factory makes instances */
	FOnlineSubsystemGoogleCommon();
	FOnlineSubsystemGoogleCommon(FName InInstanceName);

	/** @return the backend server client id */
	FString GetClientId() const { return ClientId; }

	/** @return the backend server client id */
	FString GetServerClientId() const { return ServerClientId; }

protected:

	/** The client id given to us by Google Dashboard */
	FString ClientId;

	/** Server client id that this client will be engaging with */
	FString ServerClientId;
	
	/** Does the implementation need a Client Id configured*/
	bool bPlatformRequiresClientId;

	/** Does the implementation need a Client Id configured*/
	bool bPlatformAllowsClientIdOverride;

	/** Does the implementation need a Server Client Id configured*/
	bool bPlatformRequiresServerClientId;

	/** Google implementation of identity interface */
	FOnlineIdentityGoogleCommonPtr GoogleIdentity;

	/** Google implementation of the external ui */
	FOnlineExternalUIGoogleCommonPtr GoogleExternalUI;
};

typedef TSharedPtr<FOnlineSubsystemGoogleCommon, ESPMode::ThreadSafe> FOnlineSubsystemGoogleCommonPtr;
