// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "OnlineSubsystemGooglePackage.h"

/**
 * Online subsystem module class (Google Implementation)
 * Code related to the loading and handling of the Google module.
 */
class FOnlineSubsystemGoogleModule : public IModuleInterface
{
private:

	/** Class responsible for creating instance(s) of the subsystem */
	class FOnlineFactoryGoogle* GoogleFactory;

public:	

	// IModuleInterface 

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}

	// FOnlineSubsystemGoogleModule

	/**
	 * Constructor
	 */
	FOnlineSubsystemGoogleModule() :
		GoogleFactory(nullptr)
	{}

	/**
	 * Destructor
	 */
	virtual ~FOnlineSubsystemGoogleModule() 
	{}
};


